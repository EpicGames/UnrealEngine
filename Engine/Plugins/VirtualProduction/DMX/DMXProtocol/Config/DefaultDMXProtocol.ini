[CoreRedirects]

// 4.26
+StructRedirects=(OldName="DMXUniverse",NewName="DMXCommunicationEndpoint")

+PropertyRedirects=(OldName="DMXEntityUniverseManaged.Universes",NewName="DMXEntityUniverseManaged.Endpoints")


// 4.27
+EnumRedirects=(OldName="EDMXCommunicationTypes", NewName="EDMXCommunicationType")
