{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "DMX Protocol",
	"Description": "DMX Protocols implementation",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Modules": [
		{
			"Name": "DMXProtocol",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "DMXProtocolArtNet",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "DMXProtocolSACN",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "DMXProtocolEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DMXProtocolBlueprintGraph",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	]
}
