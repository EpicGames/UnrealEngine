// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXProtocol : ModuleRules
{
	public DMXProtocol(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Projects"
			});
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Json",
				"Networking",
				"Serialization",
				"Sockets"				
			});

		if (Target.Type == TargetType.Editor)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Settings",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Settings",
				});

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"SlateCore",
				"Slate",
				"InputCore"
			}
		);
		}

	}
}
