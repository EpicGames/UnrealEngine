// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Editor only feature
#if WITH_EDITOR 

#include "Containers/Queue.h"
#include "Containers/Map.h"
#include "GenericPlatform/GenericPlatform.h"
#include "HAL/Platform.h"
#include "Tasks/Task.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FDMXInputPort;
class FDMXOutputPort;


namespace UE::DMX
{
	struct DMXPROTOCOL_API FDMXMonitoredOutboundDMXData
		: public TSharedFromThis<FDMXMonitoredOutboundDMXData>
	{
		FDMXMonitoredOutboundDMXData(
			TWeakPtr<FDMXOutputPort> InOutputPort,
			int32 InLocalUniverseID,
			TMap<int32, uint8> InChannelToValueMap,
			FName InTrace)
			: OutputPort(InOutputPort)
			, LocalUniverseID(InLocalUniverseID)
			, ChannelToValueMap(InChannelToValueMap)
			, Trace(InTrace)
		{}

		/** Returns true if this data conflicts with Other */
		bool ConflictsWith(const FDMXMonitoredOutboundDMXData& Other) const;

		const TWeakPtr<FDMXOutputPort> OutputPort;
		const int32 LocalUniverseID;
		const TMap<int32, uint8> ChannelToValueMap;

		/** The users that raised this specific trace, in callstack order. */
		const FName Trace;
	};

	/** User session for a conflict monitor user. Create via FDMXConflictMonitor::Join */
	class DMXPROTOCOL_API FDMXConflictMonitorUserSession
		: public TSharedFromThis<FDMXConflictMonitorUserSession>
	{
	public:
		~FDMXConflictMonitorUserSession();

	private:
		// Allow conflict monitor to create active instances
		friend class FDMXConflictMonitor;
		
		// Allows MakeShared with a private constructor
		friend class SharedPointerInternals::TIntrusiveReferenceController<FDMXConflictMonitorUserSession, ESPMode::ThreadSafe>;

		FDMXConflictMonitorUserSession() = default;
		FDMXConflictMonitorUserSession(FName InUserName);

		FName UserName;
	};

	/** 
	 * Monitors outbound DMX for possible conflicts. Only sees conflict created within a single port.
	 * 
	 * User objects usually will want to refer to the instance declared in port manager, see FDMXPortManager::GetConflictMonitor.
	 */
	class DMXPROTOCOL_API FDMXConflictMonitor
		: public TSharedFromThis<FDMXConflictMonitor>
	{
		// Let user sessions join and leave 
		friend class FDMXConflictMonitorUserSession;

	public:
		virtual ~FDMXConflictMonitor();

		/** 
		 * Joins monitoring for conflicts. Returns a user session.
		 * The monitor actively scans for outbound DMX conflicts while the user session is alive.
		 */
		[[nodiscard]] static TSharedRef<FDMXConflictMonitorUserSession> Join(const FName& UserName);

		/** Returns true if the conflict monitor is enabled */
		static bool IsEnabled();

		/** Returns the conflict monitor, or nullptr if it has no users (see Join to add users). */
		static FDMXConflictMonitor* Get();

		/** Returns current conflicts, locking. */
		TMap<FName, TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>> GetOutboundConflictsSynchronous() const;

		/** Returns currently monitored data. This contains non-conflicting data as well. */
		TArray<TSharedRef<FDMXMonitoredOutboundDMXData>> GetMonitoredOutboundData() const { return MonitoredOutboundData; }

		/** Adds a trace to the queue stack. */
		void TraceUser(const FMinimalName& User);

		/** Pops the last trace from the trace stack */
		void PopTrace(const FMinimalName& User);

		/** Monitors outbound DMX. Can be called from the game thread only. */
		void MonitorOutboundDMX(const TSharedRef<FDMXOutputPort>& InOutputPort, int32 InLocalUniverseID, const TMap<int32, uint8>& InChannelToValueMap);

		/** Returns the time the monitor consumes on the game thread */
		double GetTimeGameThread() const { return TimeGameThread; }

	private:
		/** Removes a user from the monitor. If all user objects left, the monitor is disabled. */
		static void RemoveUser(const FName& UserName);

		/** The currently stacked traces */
		TArray<FMinimalName> TraceStack;

		/** Channel values this frame */
		TMap<int32, uint8> ChannelToValueMap;

		/** Current outbound data */
		TArray<TSharedRef<FDMXMonitoredOutboundDMXData>> MonitoredOutboundData;

		/** An array of conflicting traces */
		TMap<FName, TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>>  Conflicts;

		/** The last frame number that was handled on the monitor thread */
		uint64 FrameNumber = 0;

		/** The time the monitor uses on the game thread */
		double TimeGameThread = 0.0;

		/** Users of the monitor */
		static TArray<FName> UserNames;

		/** Global instance. Only valid if there are users. */
		static TSharedPtr<FDMXConflictMonitor> Instance;

		/** Mutex access for the Conflicts array */
		mutable FCriticalSection MutexAccessConflicts;
	};
}

#endif // WITH_EDITOR
