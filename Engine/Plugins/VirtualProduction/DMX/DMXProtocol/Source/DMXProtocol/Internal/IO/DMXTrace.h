// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"


namespace UE::DMX
{
#if WITH_EDITOR
	namespace Private::Trace
	{
		class DMXPROTOCOL_API FDMXScopedSendDMXTrace
		{
		public:
			FDMXScopedSendDMXTrace(const FName& InUser);
			virtual ~FDMXScopedSendDMXTrace();

		private:
			FMinimalName User;
		};
	}

#define UE_DMX_SCOPED_TRACE_SENDDMX(Name) \
	using namespace UE::DMX; \
	using namespace UE::DMX::Private::Trace; \
	const FDMXScopedSendDMXTrace ANONYMOUS_VARIABLE(DMXScopedTrace)(Name); 

#else // !WITH_EDITOR
	// Opt out instead of making users wrap with #if WITH_EDITOR
#define UE_DMX_SCOPED_TRACE_SENDDMX(User)
#endif // WITH_EDITOR
}
