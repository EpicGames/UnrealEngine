// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXAttribute.h"

#include "DMXProtocolModule.h"
#include "DMXProtocolObjectVersion.h"
#include "DMXProtocolSettings.h"
#include "Modules/ModuleManager.h"

// UE-224455. Before 5.5 the Name member was defaulted to the first entry of the Attributes set in project settings, 
// which is variable implicitly. Since CDOs can define the struct default individually, it is important to keep the
// original attribute name "Color". This is to ensure CDOs of objects created before 5.5 get the correct default value.
// 
// Also see FDMXAttributeName::Serialize where instances that use a structure serializer are handled.
FDMXAttributeName::FDMXAttributeName()
	: Name("Color")
{
}

FDMXAttributeName::FDMXAttributeName(const FDMXAttribute& InAttribute)
{
	Name = InAttribute.Name;
}

FDMXAttributeName::FDMXAttributeName(const FName& NameAttribute)
{
	Name = NameAttribute;
}

void FDMXAttributeName::SetFromName(const FName& InName)
{
	*this = InName;
}

TArray<FName> FDMXAttributeName::GetPredefinedValues()
{
	TArray<FName> Result;
	const UDMXProtocolSettings* DMXSettings = GetDefault<UDMXProtocolSettings>();
	if (!DMXSettings)
	{
		return Result;
	}

	for (const FDMXAttribute& Attribute : DMXSettings->Attributes)
	{
		Result.Add(Attribute.Name);
	}
	return Result;
}

bool FDMXAttributeName::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	Ar.UsingCustomVersion(FDMXProtocolObjectVersion::GUID);
	if (Ar.CustomVer(FDMXProtocolObjectVersion::GUID) < FDMXProtocolObjectVersion::FixAttributeNameDefaultValue)
	{			
		// UE-224455. The Name member was defaulted to the first entry of the Attributes set in project settings, 
		// which is variable implicitly. For old projects keep this behaviour when upgrading to 5.5.

		const IModuleInterface* DMXProtocolModule = FModuleManager::Get().GetModule("DMXProtocol");
		if (DMXProtocolModule != nullptr)
		{
			if (const UDMXProtocolSettings* DMXSettings = GetDefault<UDMXProtocolSettings>())
			{
				if (DMXSettings->Attributes.Num() > 0)
				{
					Name = DMXSettings->Attributes.begin()->Name;
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	// Serialize is only implemented to recall the right default for versions before 5.5.
	// Return false to leave it to the outer serializer to perform the actual serialization.
	return false;
}

FString UDMXAttributeNameConversions::Conv_DMXAttributeToString(const FDMXAttributeName& InAttribute)
{
	return InAttribute.Name.ToString();
}

FName UDMXAttributeNameConversions::Conv_DMXAttributeToName(const FDMXAttributeName& InAttribute)
{
	return InAttribute.Name;
}

TArray<FString> FDMXAttribute::GetKeywords() const
{
	TArray<FString> CleanedKeywords;
	Keywords.ParseIntoArray(CleanedKeywords, TEXT(","));
	for (FString& CleanKeyword : CleanedKeywords)
	{
		CleanKeyword.TrimStartAndEndInline();
	}

	return CleanedKeywords;
}

void FDMXAttribute::CleanupKeywords()
{
	// support tabs too
	Keywords = Keywords.ConvertTabsToSpaces(1);
	Keywords.TrimStartAndEndInline();
	TArray<FString> KeywordsArray;
	Keywords.ParseIntoArray(KeywordsArray, TEXT(","), true);
	Keywords = FString::Join(KeywordsArray, TEXT(", "));
}
