// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXConversions.h"


uint8 FDMXConversions::GetSizeOfSignalFormat(EDMXFixtureSignalFormat SignalFormat)
{
	return static_cast<uint8>(SignalFormat) + 1;
}

uint32 FDMXConversions::GetSignalFormatMaxValue(EDMXFixtureSignalFormat SignalFormat)
{
	switch (SignalFormat)
	{
	case EDMXFixtureSignalFormat::E8Bit:
		return 0x000000FF;
	case EDMXFixtureSignalFormat::E16Bit:
		return 0x0000FFFF;
	case EDMXFixtureSignalFormat::E24Bit:
		return 0x00FFFFFF;
	case EDMXFixtureSignalFormat::E32Bit:
		return 0xFFFFFFFF;
	default:
		checkNoEntry(); // Unhandled enum value
	}

	return 0;
}

uint32 FDMXConversions::ClampValueBySignalFormat(uint32 Value, EDMXFixtureSignalFormat SignalFormat)
{
	return FMath::Min(Value, GetSignalFormatMaxValue(SignalFormat));
}
