// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolBlueprintLibrary.h"

#include "DMXProtocolSettings.h"
#include "DMXProtocolUtils.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXInputPortConfig.h"
#include "IO/DMXOutputPortConfig.h"
#include "IO/DMXPortManager.h"


void UDMXProtocolBlueprintLibrary::SetSendDMXEnabled(bool bSendDMXEnabled, bool bAffectEditor)
{
	if (bAffectEditor || !GIsEditor)
	{
		UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
		ProtocolSettings->OverrideSendDMXEnabled(bSendDMXEnabled);		
	}
}

bool UDMXProtocolBlueprintLibrary::IsSendDMXEnabled()
{
	const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
	return ProtocolSettings->IsSendDMXEnabled();
}

void UDMXProtocolBlueprintLibrary::SetReceiveDMXEnabled(bool bReceiveDMXEnabled, bool bAffectEditor)
{
	if (bAffectEditor || !GIsEditor)
	{
		UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
		ProtocolSettings->OverrideReceiveDMXEnabled(bReceiveDMXEnabled);
	}
}

bool UDMXProtocolBlueprintLibrary::IsReceiveDMXEnabled()
{
	const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
	return ProtocolSettings->IsReceiveDMXEnabled();
}

TArray<FString> UDMXProtocolBlueprintLibrary::GetLocalDMXNetworkInterfaceCardIPs()
{
	TArray<FString> NetworkInterfaceCardIPs;

	TArray<TSharedPtr<FString>> NetworkInterfaceCardIPSharedPtrs = FDMXProtocolUtils::GetLocalNetworkInterfaceCardIPs();

	for (const TSharedPtr<FString>& IP : NetworkInterfaceCardIPSharedPtrs)
	{
		if (IP.IsValid())
		{
			NetworkInterfaceCardIPs.Add(*IP);
		}
	}

	return NetworkInterfaceCardIPs;
}

void UDMXProtocolBlueprintLibrary::SetDMXInputPortDeviceAddress(FDMXInputPortReference InputPort, const FString& DeviceAddress)
{
	UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();

	if (ProtocolSettings)
	{
		FDMXInputPortConfig* InputPortConfigPtr = ProtocolSettings->InputPortConfigs.FindByPredicate([&InputPort](const FDMXInputPortConfig& InputPortConfig)
			{
				return InputPortConfig.GetPortGuid() == InputPort.GetPortGuid();
			});

		if (InputPortConfigPtr)
		{
			FDMXInputPortConfigParams InputPortConfigParams(*InputPortConfigPtr);
			InputPortConfigParams.DeviceAddress = DeviceAddress;

			*InputPortConfigPtr = FDMXInputPortConfig(InputPortConfigPtr->GetPortGuid(), InputPortConfigParams);

			FDMXPortManager::Get().UpdateFromProtocolSettings();
		}
	}
}

void UDMXProtocolBlueprintLibrary::SetDMXOutputPortDeviceAddress(FDMXOutputPortReference OutputPort, const FString& DeviceAddress)
{
	UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();

	if (ProtocolSettings)
	{
		FDMXOutputPortConfig* OutputPortConfigPtr = ProtocolSettings->OutputPortConfigs.FindByPredicate([&OutputPort](const FDMXOutputPortConfig& OutputPortConfig)
			{
				return OutputPortConfig.GetPortGuid() == OutputPort.GetPortGuid();
			});

		if (OutputPortConfigPtr)
		{
			FDMXOutputPortConfigParams OutputPortConfigParams(*OutputPortConfigPtr);
			OutputPortConfigParams.DeviceAddress = DeviceAddress;

			*OutputPortConfigPtr = FDMXOutputPortConfig(OutputPortConfigPtr->GetPortGuid(), OutputPortConfigParams);

			FDMXPortManager::Get().UpdateFromProtocolSettings();
		}
	}
}

void UDMXProtocolBlueprintLibrary::SetDMXOutputPortDestinationAddresses(FDMXOutputPortReference OutputPort, const TArray<FString>& DestinationAddresses)
{
	UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();

	if (ProtocolSettings)
	{
		FDMXOutputPortConfig* OutputPortConfigPtr = ProtocolSettings->OutputPortConfigs.FindByPredicate([&OutputPort](const FDMXOutputPortConfig& OutputPortConfig)
			{
				return OutputPortConfig.GetPortGuid() == OutputPort.GetPortGuid();
			});

		if (OutputPortConfigPtr)
		{
			TArray<FDMXOutputPortDestinationAddress> DestinationAddressesStructs;
			for (const FString& DestinationAddress : DestinationAddresses)
			{
				DestinationAddressesStructs.Add(FDMXOutputPortDestinationAddress(DestinationAddress));
			}

			FDMXOutputPortConfigParams OutputPortConfigParams(*OutputPortConfigPtr);
			OutputPortConfigParams.DestinationAddresses = DestinationAddressesStructs;

			*OutputPortConfigPtr = FDMXOutputPortConfig(OutputPortConfigPtr->GetPortGuid(), OutputPortConfigParams);

			FDMXPortManager::Get().UpdateFromProtocolSettings();
		}
	}
}

void UDMXProtocolBlueprintLibrary::SetDMXOutputPortDestinationAddress(FDMXOutputPortReference OutputPort, const FString& DestinationAddress)
{
	// DEPRECATED 5.0

	UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();

	if (ProtocolSettings)
	{
		FDMXOutputPortConfig* OutputPortConfigPtr = ProtocolSettings->OutputPortConfigs.FindByPredicate([&OutputPort](const FDMXOutputPortConfig& OutputPortConfig)
			{
				return OutputPortConfig.GetPortGuid() == OutputPort.GetPortGuid();
			});

		if (OutputPortConfigPtr)
		{
			FDMXOutputPortConfigParams OutputPortConfigParams(*OutputPortConfigPtr);

			if (OutputPortConfigParams.DestinationAddresses.Num() > 0)
			{
				OutputPortConfigParams.DestinationAddresses[0] = FDMXOutputPortDestinationAddress(DestinationAddress);
			}
			else
			{
				OutputPortConfigParams.DestinationAddresses.Add(FDMXOutputPortDestinationAddress(DestinationAddress));
			}

			*OutputPortConfigPtr = FDMXOutputPortConfig(OutputPortConfigPtr->GetPortGuid(), OutputPortConfigParams);

			FDMXPortManager::Get().UpdateFromProtocolSettings();
		}
	}
}
