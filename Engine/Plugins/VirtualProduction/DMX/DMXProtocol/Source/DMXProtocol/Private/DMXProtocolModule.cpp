// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolModule.h"

#include "DMXProtocolLog.h"
#include "DMXProtocolSettings.h"
#include "Interfaces/IDMXProtocol.h"
#include "Interfaces/IDMXProtocolFactory.h"
#include "IO/DMXPortManager.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#endif // WITH_EDITOR

#include "Interfaces/IPluginManager.h"


IMPLEMENT_MODULE( FDMXProtocolModule, DMXProtocol );

#define LOCTEXT_NAMESPACE "DMXProtocolModule"

const FName FDMXProtocolModule::DefaultProtocolArtNetName = "Art-Net";
const FName FDMXProtocolModule::DefaultProtocolSACNName = "sACN";

FDMXProtocolModule::FDMXOnRequestProtocolRegistrationEvent FDMXProtocolModule::OnRequestProtocolRegistrationEvent;
FDMXProtocolModule::FDMXOnRequestProtocolBlocklistEvent FDMXProtocolModule::OnRequestProtocolBlocklistEvent;

FDMXProtocolModule::FDMXOnRequestProtocolRegistrationEvent& FDMXProtocolModule::GetOnRequestProtocolRegistration()
{
	return OnRequestProtocolRegistrationEvent;
}

FDMXProtocolModule::FDMXOnRequestProtocolBlocklistEvent& FDMXProtocolModule::GetOnRequestProtocolBlocklist()
{
	return OnRequestProtocolBlocklistEvent;
}

void FDMXProtocolModule::RegisterProtocol(const FName& ProtocolName, IDMXProtocolFactory* Factory)
{
	// DEPRECATED 4.27
	if (!DMXProtocolFactories.Contains(ProtocolName))
	{
		// Add a factory for the protocol
		DMXProtocolFactories.Add(ProtocolName, Factory);
	}
	else
	{
		UE_LOG_DMXPROTOCOL(Verbose, TEXT("Trying to add existing protocol %s"), *ProtocolName.ToString());
	}

	IDMXProtocolPtr NewProtocol = Factory->CreateProtocol(ProtocolName);
	if (NewProtocol.IsValid())
	{
		DMXProtocols.Add(ProtocolName, NewProtocol);

		UE_LOG_DMXPROTOCOL(Log, TEXT("Creating protocol instance for: %s"), *ProtocolName.ToString());
	}
	else
	{
		UE_LOG_DMXPROTOCOL(Verbose, TEXT("Unable to create Protocol %s"), *ProtocolName.ToString());
	}
}

void FDMXProtocolModule::UnregisterProtocol(const FName& ProtocolName)
{
	if (DMXProtocolFactories.Contains(ProtocolName))
	{
		// Destroy the factory and shut down the protocol
		DMXProtocolFactories.Remove(ProtocolName);
		ShutdownDMXProtocol(ProtocolName);
	}
	else
	{
		UE_LOG_DMXPROTOCOL(Verbose, TEXT("Trying to remove unexisting protocol %s"), *ProtocolName.ToString());
	}
}

FDMXProtocolModule& FDMXProtocolModule::Get()
{
	return FModuleManager::GetModuleChecked<FDMXProtocolModule>("DMXProtocol");
}

IDMXProtocolPtr FDMXProtocolModule::GetProtocol(const FName InProtocolName)
{
	IDMXProtocolPtr* ProtocolPtr = DMXProtocols.Find(InProtocolName);

	return ProtocolPtr ? *ProtocolPtr : nullptr;
}

const TMap<FName, IDMXProtocolFactory*>& FDMXProtocolModule::GetProtocolFactories() const
{
	return DMXProtocolFactories;
}

const TMap<FName, IDMXProtocolPtr>& FDMXProtocolModule::GetProtocols() const
{
	return DMXProtocols;
}

void FDMXProtocolModule::StartupModule()
{
#if WITH_EDITOR
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	// Register DMX Protocol global settings
	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "DMX",
			LOCTEXT("ProjectSettings_Label", "DMX"),
			LOCTEXT("ProjectSettings_Description", "Configure DMX plugin global settings"),
			GetMutableDefault<UDMXProtocolSettings>()
		);
	}
#endif // WITH_EDITOR

	IPluginManager& PluginManager = IPluginManager::Get();
	PluginManager.OnLoadingPhaseComplete().AddRaw(this, &FDMXProtocolModule::OnPluginLoadingPhaseComplete);
}

void FDMXProtocolModule::ShutdownModule()
{
	IPluginManager& PluginManager = IPluginManager::Get();
	PluginManager.OnLoadingPhaseComplete().RemoveAll(this);

	FDMXPortManager::ShutdownManager();

	// Now Shutdown the protocols
	ShutdownAllDMXProtocols();

#if WITH_EDITOR
	// Unregister DMX Protocol global settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "DMX Protocol");
	}
#endif // WITH_EDITOR
}

void FDMXProtocolModule::OnPluginLoadingPhaseComplete(ELoadingPhase::Type LoadingPhase, bool bPhaseSuccessful)
{
	if (bPhaseSuccessful && LoadingPhase == ELoadingPhase::PreDefault)
	{
		// Request other plugins to provide their blocked protocols
		TArray<FName> ProtocolBlocklist;
		GetOnRequestProtocolBlocklist().Broadcast(ProtocolBlocklist);

		// Request protocols to register themselves
		TArray<FDMXProtocolRegistrationParams> ProtocolRegistrationParamsArray;
		GetOnRequestProtocolRegistration().Broadcast(ProtocolRegistrationParamsArray);

		// Sort to make the default protocols show topmost
		ProtocolRegistrationParamsArray.Sort([](const FDMXProtocolRegistrationParams& ParamsA, const FDMXProtocolRegistrationParams& ParamsB) {
			return
				(ParamsA.ProtocolName == DefaultProtocolArtNetName) ||													// Art-Net first if available
				(ParamsB.ProtocolName != DefaultProtocolArtNetName && ParamsA.ProtocolName == DefaultProtocolSACNName);	// sACN 2nd if available
			});

		// Register all protocols that joined in
		for (const FDMXProtocolRegistrationParams& RegistrationParams : ProtocolRegistrationParamsArray)
		{
			if (ProtocolBlocklist.Contains(RegistrationParams.ProtocolName))
			{
				UE_LOG_DMXPROTOCOL(Log, TEXT("Blocked registration of DMX Protocol %s as requested per Protocol Blocklist."), *RegistrationParams.ProtocolName.ToString());
			}
			else if (ensureMsgf(RegistrationParams.ProtocolFactory, TEXT("No Protocol Factory provided for Protocol %s, protocol cannot be loaded."), *RegistrationParams.ProtocolName.ToString()))
			{
				IDMXProtocolPtr NewProtocol = RegistrationParams.ProtocolFactory->CreateProtocol(RegistrationParams.ProtocolName);
				if (NewProtocol.IsValid())
				{
					DMXProtocolFactories.Add(RegistrationParams.ProtocolName, RegistrationParams.ProtocolFactory);
					DMXProtocols.Add(RegistrationParams.ProtocolName, NewProtocol);
				}
				else
				{
					UE_LOG_DMXPROTOCOL(Verbose, TEXT("Unable to create Protocol %s"), *RegistrationParams.ProtocolName.ToString());
				}
			}
		}

		// Startup and update the manager, alike  in the Default loading phase the manager is fully available, including Ports from Project Settings.
		FDMXPortManager::StartupManager();
		FDMXPortManager::Get().UpdateFromProtocolSettings();

		// Don't let the binding fire twice for whatever reason
		IPluginManager& PluginManager = IPluginManager::Get();
		PluginManager.OnLoadingPhaseComplete().RemoveAll(this);
	}
}

void FDMXProtocolModule::ShutdownDMXProtocol(const FName& ProtocolName)
{
	if (!ProtocolName.IsNone())
	{
		IDMXProtocolPtr DMXProtocol;
		DMXProtocols.RemoveAndCopyValue(ProtocolName, DMXProtocol);
		if (DMXProtocol.IsValid())
		{
			DMXProtocol->Shutdown();
		}
		else
		{
			UE_LOG_DMXPROTOCOL(Verbose, TEXT("DMXProtocol instance %s not found, unable to destroy."), *ProtocolName.ToString());
		}
	}
}

void FDMXProtocolModule::ShutdownAllDMXProtocols()
{
	for (TMap<FName, IDMXProtocolPtr>::TIterator It = DMXProtocols.CreateIterator(); It; ++It)
	{
		It->Value->Shutdown();
	}
}

#undef LOCTEXT_NAMESPACE
