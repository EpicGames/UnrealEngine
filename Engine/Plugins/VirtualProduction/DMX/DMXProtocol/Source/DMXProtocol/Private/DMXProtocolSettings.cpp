// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolSettings.h"

#include "DMXProtocolBlueprintLibrary.h"
#include "DMXProtocolConstants.h"
#include "DMXProtocolObjectVersion.h"
#include "IO/DMXPortManager.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXPortManager.h"

#include "IPAddress.h"
#include "SocketSubsystem.h"


UDMXProtocolSettings::UDMXProtocolSettings()
	: SendingRefreshRate(DMX_RATE)
	, ReceivingRefreshRate_DEPRECATED(DMX_RATE)
{
	FixtureCategories =
	{
		TEXT("Static"),
		TEXT("Matrix/Pixel Bar"),
		TEXT("Moving Head"),
		TEXT("Moving Mirror"),
		TEXT("Strobe"),
		TEXT("Other")
	};

	// Default Attributes
	Attributes =
	{
		/* Fixture related */

		// Label					Keywords
		{ TEXT("Color"),			TEXT("ColorWheel, Color1") },
		{ TEXT("Red"),				TEXT("ColorAdd_R") },
		{ TEXT("Green"),			TEXT("ColorAdd_G") },
		{ TEXT("Blue"),				TEXT("ColorAdd_B") },
		{ TEXT("Cyan"),				TEXT("ColorAdd_C, ColorSub_C") },
		{ TEXT("Magenta"),			TEXT("ColorAdd_M, ColorSub_M") },
		{ TEXT("Yellow"),			TEXT("ColorAdd_Y, ColorSub_Y") },
		{ TEXT("White"),			TEXT("ColorAdd_W") },
		{ TEXT("Amber"),			TEXT("ColorAdd_A") },
		{ TEXT("Dimmer"),			TEXT("Intensity, Strength, Brightness") },
		{ TEXT("Pan"),				TEXT("") },
		{ TEXT("Shutter"),			TEXT("Strobe") },
		{ TEXT("Tilt"),				TEXT("") },
		{ TEXT("Zoom"),				TEXT("") },
		{ TEXT("Focus"),			TEXT("") },
		{ TEXT("Iris"),				TEXT("") },
		{ TEXT("Gobo"),				TEXT("GoboWheel, Gobo1") },
		{ TEXT("Gobo Spin"),		TEXT("GoboSpin") },
		{ TEXT("Gobo Wheel Rotate"),TEXT("GoboWheelSpin, GoboWheelRotate") },
		{ TEXT("Color Rotation"),	TEXT("ColorWheelSpin") },
		{ TEXT("Shaper Rotation"),	TEXT("ShaperRot") },
		{ TEXT("Effects"),			TEXT("Effect, Macro, Effects") },
		{ TEXT("Frost"),			TEXT("") },
		{ TEXT("Reset"),			TEXT("FixtureReset, FixtureGlobalReset, GlobalReset") },
		{ TEXT("CTC"),				TEXT("") },
		{ TEXT("Tint"),				TEXT("") },
		{ TEXT("Color XF"),			TEXT("") },
		{ TEXT("HSB_Hue"),			TEXT("") },
		{ TEXT("HSB_Saturation"),	TEXT("") },
		{ TEXT("HSB_Brightness"),	TEXT("") },
		{ TEXT("FanMode"),			TEXT("") },
		{ TEXT("CIE_X"),			TEXT("") },
		{ TEXT("CIE_Y"),			TEXT("") },
		{ TEXT("Prism"),			TEXT("") },

		/* Firework, Fountain related */

		// Label					Keywords
		{ TEXT("ModeStartStop"),	TEXT("") },
		{ TEXT("Burst"),			TEXT("") },
		{ TEXT("Launch"),			TEXT("") },
		{ TEXT("Velocity"),			TEXT("") },
		{ TEXT("Angle"),			TEXT("") },
		{ TEXT("NumBeams"),			TEXT("") }
	};
}

void UDMXProtocolSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Force cleanup of the keywords on load
	// This is required for supporting previous implementations where spaces were used
	for (FDMXAttribute& Attribute : Attributes)
	{
		Attribute.CleanupKeywords();
	}

	// Parse command line options for send and receive dmx
	if (FParse::Bool(FCommandLine::Get(), TEXT("DEFAULTSENDDMXENABLED="), bDefaultSendDMXEnabled))
	{
		UE_LOGF(LogDMXProtocol, Log, "Overridden Default Send DMX Enabled from command line, set to %ls.", bDefaultSendDMXEnabled ? TEXT("True") : TEXT("False"));
	}
	OverrideSendDMXEnabled(bDefaultSendDMXEnabled);

	if (FParse::Bool(FCommandLine::Get(), TEXT("DEFAULTRECEIVEDMXENABLED="), bDefaultReceiveDMXEnabled))
	{
		UE_LOGF(LogDMXProtocol, Log, "Overridden Default Receive DMX Enabled from command line, set to %ls.", bDefaultReceiveDMXEnabled ? TEXT("True") : TEXT("False"));
	}
	OverrideReceiveDMXEnabled(bDefaultReceiveDMXEnabled);

	// Upgrade from single to many destination addresses for ports for projects created before 4.27
	const int32 CustomVersion = GetLinkerCustomVersion(FDMXProtocolObjectVersion::GUID);
	if (CustomVersion < FDMXProtocolObjectVersion::OutputPortSupportsManyUnicastAddresses)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		TSharedPtr<FInternetAddr> InternetAddr = SocketSubsystem->CreateInternetAddr();

		for (FDMXOutputPortConfig& OutputPortConfig : OutputPortConfigs)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FString DestinationAddress = OutputPortConfig.GetDestinationAddress();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			bool bIsValidIP = false;
			InternetAddr->SetIp(*DestinationAddress, bIsValidIP);
			if (bIsValidIP)
			{
				FDMXOutputPortConfigParams NewPortParams = FDMXOutputPortConfigParams(OutputPortConfig);
				NewPortParams.DestinationAddresses = { DestinationAddress };

				OutputPortConfig = FDMXOutputPortConfig(OutputPortConfig.GetPortGuid(), NewPortParams);
			}
		}
	}

	// Allow port configs to take their own upgrade path
	for (FDMXInputPortConfig& InputPortConfig : InputPortConfigs)
	{
		InputPortConfig.Upgrade();
	}

	for (FDMXOutputPortConfig& OutputPortConfig : OutputPortConfigs)
	{
		OutputPortConfig.Upgrade();
	}
}

#if WITH_EDITOR
void UDMXProtocolSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, bDefaultReceiveDMXEnabled))
	{
		bOverrideReceiveDMXEnabled = bDefaultReceiveDMXEnabled;
		UDMXProtocolBlueprintLibrary::SetReceiveDMXEnabled(bDefaultReceiveDMXEnabled);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, bDefaultSendDMXEnabled))
	{
		bOverrideSendDMXEnabled = bDefaultSendDMXEnabled;
		UDMXProtocolBlueprintLibrary::SetSendDMXEnabled(bDefaultSendDMXEnabled);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, bAllFixturePatchesReceiveDMXInEditor))
	{
		OnAllFixturePatchesReceiveDMXInEditorEnabled.Broadcast(bAllFixturePatchesReceiveDMXInEditor);
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXProtocolSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	const FName PropertyName = PropertyChangedChainEvent.GetPropertyName();
	const FProperty* Property = PropertyChangedChainEvent.Property;
	const UScriptStruct* InputPortConfigStruct = FDMXInputPortConfig::StaticStruct();
	const UScriptStruct* OutputPortConfigStruct = FDMXOutputPortConfig::StaticStruct();
	const UStruct* PropertyOwnerStruct = Property ? Property->GetOwnerStruct() : nullptr;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, SendingRefreshRate))
	{
		OnSendingRefresRateChanged.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, FixtureCategories))
	{
		if (FixtureCategories.Num() == 0)
		{
			FixtureCategories.Add(TEXT("Other"));
		}

		OnDefaultFixtureCategoriesChanged.Broadcast();
	}
	else if (
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, Attributes) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FDMXAttribute, Name) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FDMXAttribute, Keywords))
	{
		if (Attributes.Num() == 0)
		{
			Attributes.Add({ NAME_None, TEXT("") });
		}

		for (FDMXAttribute& Attribute : Attributes)
		{
			Attribute.CleanupKeywords();
		}

		OnDefaultAttributesChanged.Broadcast();
	}
	else if (
		PropertyName == FDMXOutputPortConfig::GetDestinationAddressesPropertyNameChecked() ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FDMXOutputPortDestinationAddress, DestinationAddressString))
	{
		FDMXPortManager::Get().UpdateFromProtocolSettings();
	}
	else if (
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, InputPortConfigs) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, OutputPortConfigs) ||
		(InputPortConfigStruct && InputPortConfigStruct == PropertyOwnerStruct) ||
		(OutputPortConfigStruct && OutputPortConfigStruct == PropertyOwnerStruct))
	{
		// When duplicating configs, the guid will be duplicated, so we have to create unique ones instead
		if (PropertyChangedChainEvent.ChangeType == EPropertyChangeType::Duplicate)
		{
			int32 ChangedIndex = PropertyChangedChainEvent.GetArrayIndex(PropertyName.ToString());
			if (ensureAlways(ChangedIndex != INDEX_NONE))
			{
				if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, InputPortConfigs))
				{
					const int32 IndexOfDuplicate = InputPortConfigs.FindLastByPredicate([this, ChangedIndex](const FDMXInputPortConfig& InputPortConfig) {
						return InputPortConfigs[ChangedIndex].GetPortGuid() == InputPortConfig.GetPortGuid();
						});

					if (ensureAlways(IndexOfDuplicate != ChangedIndex))
					{
						InputPortConfigs[IndexOfDuplicate] = FDMXInputPortConfig(FGuid::NewGuid(), InputPortConfigs[ChangedIndex]);
					}
				}
				else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, OutputPortConfigs))
				{
					const int32 IndexOfDuplicate = OutputPortConfigs.FindLastByPredicate([this, ChangedIndex](const FDMXOutputPortConfig& OutputPortConfig) {
						return OutputPortConfigs[ChangedIndex].GetPortGuid() == OutputPortConfig.GetPortGuid();
						});

					if (ensureAlways(IndexOfDuplicate != ChangedIndex))
					{
						OutputPortConfigs[IndexOfDuplicate] = FDMXOutputPortConfig(FGuid::NewGuid(), OutputPortConfigs[ChangedIndex]);
					}
				}
			}
		}

		if (PropertyChangedChainEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			constexpr bool bForceUpdateRegistrationWithProtocol = true;
			FDMXPortManager::Get().UpdateFromProtocolSettings(bForceUpdateRegistrationWithProtocol);
		}
	}

	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);
}
#endif // WITH_EDITOR

void UDMXProtocolSettings::OverrideSendDMXEnabled(bool bEnabled)
{
	bOverrideSendDMXEnabled = bEnabled;

	OnSetSendDMXEnabledDelegate.Broadcast(bEnabled);

	// OnSetSendDMXEnabled is deprecated 5.1 and can be removed in a future release
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OnSetSendDMXEnabled.Broadcast(bEnabled);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDMXProtocolSettings::OverrideReceiveDMXEnabled(bool bEnabled)
{
	bOverrideReceiveDMXEnabled = bEnabled;

	OnSetReceiveDMXEnabledDelegate.Broadcast(bEnabled);

	// OnSetReceiveDMXEnabled is deprecated 5.1 and can be removed in a future release
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OnSetReceiveDMXEnabled.Broadcast(bEnabled);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FString UDMXProtocolSettings::GetUniqueInputPortName() const
{
	const FString AutoNamePrefix = "InputPort";
	int32 HighestPortNumber = 0;

	for (const FDMXInputPortConfig& InputPortConfig : InputPortConfigs)
	{
		const FString& PortName = InputPortConfig.GetPortName();
		if (PortName.StartsWith(AutoNamePrefix))
		{
			int32 PortNameNumericSuffix = 0;
			if (LexTryParseString<int32>(PortNameNumericSuffix, *PortName.RightChop(AutoNamePrefix.Len())))
			{
				if (PortNameNumericSuffix > HighestPortNumber)
				{
					HighestPortNumber = PortNameNumericSuffix;
				}
			}
		}
	}

	return AutoNamePrefix + FString::FromInt(HighestPortNumber + 1);
}

FString UDMXProtocolSettings::GetUniqueOutputPortName() const
{
	const FString AutoNamePrefix = "OutputPort";
	int32 HighestPortNumber = 0;

	for (const FDMXOutputPortConfig& OutputPortConfig : OutputPortConfigs)
	{
		const FString& PortName = OutputPortConfig.GetPortName();
		if (PortName.StartsWith(AutoNamePrefix))
		{
			int32 PortNameNumericSuffix = 0;
			if (LexTryParseString<int32>(PortNameNumericSuffix, *PortName.RightChop(AutoNamePrefix.Len())))
			{
				if (PortNameNumericSuffix > HighestPortNumber)
				{
					HighestPortNumber = PortNameNumericSuffix;
				}
			}
		}
	}

	return AutoNamePrefix + FString::FromInt(HighestPortNumber + 1);
}
