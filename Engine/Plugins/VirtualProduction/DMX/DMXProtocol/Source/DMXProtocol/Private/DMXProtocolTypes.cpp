// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolTypes.h"

#include "DMXProtocolSettings.h"
#include "Interfaces/IDMXProtocol.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS
const bool FDMXProtocolName::bCanBeNone = false;
FSimpleMulticastDelegate FDMXProtocolName::OnValuesChanged;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FDMXProtocolName::FDMXProtocolName(IDMXProtocolPtr InProtocol)
{
	if (InProtocol.IsValid())
	{
		Name = InProtocol->GetProtocolName();
	}
}

FDMXProtocolName::FDMXProtocolName(const FName& InName)
{
	Name = InName;
}

FDMXProtocolName::FDMXProtocolName()
{
	// GetFirstProtocolName depends on the FDMXProtocolModule.
	// This can be called on CDO creation, when the module might not be available yet.
	// So we first check if it is available.
	const IModuleInterface* DMXProtocolModule = FModuleManager::Get().GetModule("DMXProtocol");
	if (DMXProtocolModule != nullptr)
	{
		Name = IDMXProtocol::GetFirstProtocolName();
		return;
	}

	Name = NAME_None;
}

IDMXProtocolPtr FDMXProtocolName::GetProtocol() const
{
	if (Name.IsNone())
	{
		return nullptr; 
	}
	return IDMXProtocol::Get(Name);
}

TArray<FName> FDMXProtocolName::GetPossibleValues()
{
	// DEPRECATED 5.1
	return IDMXProtocol::GetProtocolNames();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const bool FDMXFixtureCategory::bCanBeNone = false;
FSimpleMulticastDelegate FDMXFixtureCategory::OnValuesChanged;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FName FDMXFixtureCategory::GetFirstValue()
{
	const TSet<FName>& FixtureCategories = GetDefault<UDMXProtocolSettings>()->FixtureCategories;

	if (const FName* FixtureCategory = FixtureCategories.FindArbitraryElement())
	{
		return *FixtureCategory;
	}

	return FName();
}

TArray<FName> FDMXFixtureCategory::GetPredefinedValues()
{
	const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
	if (ProtocolSettings)
	{
		return ProtocolSettings->FixtureCategories.Array();
	}
	return TArray<FName>();
}

TArray<FName> FDMXFixtureCategory::GetPossibleValues()
{
	// Deprecated 5.1
	return GetPredefinedValues();
}

FDMXFixtureCategory::FDMXFixtureCategory()
{
	Name = GetFirstValue();
}

FDMXFixtureCategory::FDMXFixtureCategory(const FName& InName)
{
	Name = InName;
}

FString UDMXNameContainersConversions::Conv_DMXProtocolNameToString(const FDMXProtocolName & InProtocolName)
{
	return InProtocolName.Name.ToString();
}

FName UDMXNameContainersConversions::Conv_DMXProtocolNameToName(const FDMXProtocolName & InProtocolName)
{
	return InProtocolName.Name;
}

FString UDMXNameContainersConversions::Conv_DMXFixtureCategoryToString(const FDMXFixtureCategory & InFixtureCategory)
{
	return InFixtureCategory.Name.ToString();
}

FName UDMXNameContainersConversions::Conv_DMXFixtureCategoryToName(const FDMXFixtureCategory & InFixtureCategory)
{
	return InFixtureCategory.Name;
}
