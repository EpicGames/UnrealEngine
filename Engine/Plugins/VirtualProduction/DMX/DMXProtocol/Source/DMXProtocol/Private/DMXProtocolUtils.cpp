// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolUtils.h"

#include "IPAddress.h"
#include "SocketSubsystem.h" 


#define LOCTEXT_NAMESPACE "FDMXProtocolUtils"

TArray<TSharedPtr<FString>> FDMXProtocolUtils::GetLocalNetworkInterfaceCardIPs()
{
	TArray<TSharedPtr<FString>> LocalNetworkInterfaceCardIPs;

	// Add the default route IP Address
	const FString DefaultRouteLocalAdapterAddress = TEXT("0.0.0.0");
	LocalNetworkInterfaceCardIPs.Add(MakeShared<FString>(DefaultRouteLocalAdapterAddress));

	// Add the local host IP address
	const FString LocalHostIpAddress = TEXT("127.0.0.1");
	LocalNetworkInterfaceCardIPs.Add(MakeShared<FString>(LocalHostIpAddress));

	TArray<TSharedPtr<FInternetAddr>> Addresses;
	ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(Addresses);
	bool bFoundLocalHost = false;
	for (TSharedPtr<FInternetAddr> Address : Addresses)
	{
		// Add unique, so in ase the OS call returns with the local host or default route IP, we don't add it twice
		LocalNetworkInterfaceCardIPs.AddUnique(MakeShared<FString>(Address->ToString(false)));
	}

	return LocalNetworkInterfaceCardIPs;
}

TSharedPtr<FInternetAddr> FDMXProtocolUtils::CreateInternetAddr(const FString& IPAddress, int32 Port)
{
	if (IPAddress.IsEmpty())
	{
		return nullptr;
	}

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	TSharedPtr<FInternetAddr> InternetAddr = SocketSubsystem->CreateInternetAddr();

	bool bIsValidIP = false;
	InternetAddr->SetIp(*IPAddress, bIsValidIP);
	if (!bIsValidIP)
	{
		return nullptr;
	}

	InternetAddr->SetPort(Port);
	return InternetAddr;
}

bool FDMXProtocolUtils::FindLocalNetworkInterfaceCardIPAddress(const FString& InIPAddressWithWildcards, FString& OutLocalNetworkInterfaceCardIPAddress)
{
	TArray<FString> SearchStrings;
	constexpr bool bCullEmpty = true;
	InIPAddressWithWildcards.ParseIntoArray(SearchStrings, TEXT("."), bCullEmpty);

	const TArray<TSharedPtr<FString>> LocalNetworkInterfaceCardIPs = FDMXProtocolUtils::GetLocalNetworkInterfaceCardIPs();

	FString NewIPAddress;
	for (const TSharedPtr<FString>& NetworkInterfaceCardIP : LocalNetworkInterfaceCardIPs)
	{
		TArray<FString> NetworkInterfaceCardIPSubstrings;
		NetworkInterfaceCardIP->ParseIntoArray(NetworkInterfaceCardIPSubstrings, TEXT("."));

		// Try to match each search string
		bool bMatchesSearchStrings = true;
		for (int32 SubstringIndex = 0; SubstringIndex < NetworkInterfaceCardIPSubstrings.Num(); SubstringIndex++)
		{
			if (SearchStrings.IsValidIndex(SubstringIndex))
			{
				if (!NetworkInterfaceCardIPSubstrings[SubstringIndex].MatchesWildcard(SearchStrings[SubstringIndex]))
				{
					bMatchesSearchStrings = false;
					break;
				}
			}
		}

		if (bMatchesSearchStrings)
		{
			OutLocalNetworkInterfaceCardIPAddress = *NetworkInterfaceCardIP;
			return true;
		}
	}

	return false;
}

FString FDMXProtocolUtils::GenerateUniqueNameFromExisting(const TSet<FString>& InExistingNames, const FString& InBaseName)
{
	if (!InBaseName.IsEmpty() && !InExistingNames.Contains(InBaseName))
	{
		return InBaseName;
	}

	FString FinalName;
	FString BaseName;

	int32 Index = 0;
	if (InBaseName.IsEmpty())
	{
		BaseName = TEXT("Default");
	}
	else
	{
		// If there's an index at the end of the name, start from there
		FDMXProtocolUtils::GetNameAndIndexFromString(InBaseName, BaseName, Index);
	}

	int32 Count = (Index == 0) ? 1 : Index;
	FinalName = BaseName;
	// Add Count to the BaseName, increasing Count, until it's a non-existent name
	do
	{
		// Calculate the number of digits in the number, adding 2 (1 extra to correctly count digits, another to account for the '_' that will be added to the name
		int32 CountLength = Count > 0 ? (int32)FGenericPlatformMath::LogX(10.0f, Count) + 2 : 2;

		// If the length of the final string will be too long, cut off the end so we can fit the number
		if (CountLength + BaseName.Len() >= NAME_SIZE)
		{
			BaseName = BaseName.Left(NAME_SIZE - CountLength - 1);
		}

		FinalName = FString::Printf(TEXT("%s_%d"), *BaseName, Count);
		++Count;
	} while (InExistingNames.Contains(FinalName));

	return FinalName;
}

bool FDMXProtocolUtils::GetNameAndIndexFromString(const FString& InString, FString& OutName, int32& OutIndex)
{
	OutName = InString.TrimEnd();

	// If there's an index at the end of the name, erase it
	int32 DigitIndex = OutName.Len();
	while (DigitIndex > 0 && OutName[DigitIndex - 1] >= '0' && OutName[DigitIndex - 1] <= '9')
	{
		--DigitIndex;
	}

	bool bHadIndex = false;
	if (DigitIndex < OutName.Len() && DigitIndex > -1)
	{
		OutIndex = FCString::Atoi(*OutName.RightChop(DigitIndex));
		OutName.LeftInline(DigitIndex);
		bHadIndex = true;
	}
	else
	{
		OutIndex = 0;
	}

	// Remove separator characters at the end of the string
	OutName = OutName.TrimEnd();
	DigitIndex = OutName.Len(); // reuse this variable for the separator index

	while (DigitIndex > 0
		&& (OutName[DigitIndex  - 1] == '_'
		|| OutName[DigitIndex - 1] == '.'
		|| OutName[DigitIndex - 1] == '-'))
	{
		--DigitIndex;
	}

	if (DigitIndex < OutName.Len() && DigitIndex > -1)
	{
		OutName.LeftInline(DigitIndex);
	}

	return bHadIndex;
}

#undef LOCTEXT_NAMESPACE
