// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "IO/DMXConflictMonitor.h"

#include "Algo/Accumulate.h"
#include "Algo/Find.h"
#include "DMXProtocolConstants.h"
#include "IO/DMXOutputPort.h"
#include "DMXStats.h"

DECLARE_CYCLE_STAT(TEXT("DMX Conflict Monitor"), STAT_DMXConflictMonitor, STATGROUP_DMX);

namespace UE::DMX
{
	bool FDMXMonitoredOutboundDMXData::ConflictsWith(const FDMXMonitoredOutboundDMXData& Other) const
	{
		if (&Other == this ||
			Other.LocalUniverseID != LocalUniverseID)
		{
			return false;
		}

		// Test if the conflict is occuring in the same device at the same address
		const TSharedPtr<FDMXOutputPort> PinnedOutputPort = OutputPort.Pin();
		const TSharedPtr<FDMXOutputPort> OtherPinnedOutputPort = OutputPort.Pin();
		if (!PinnedOutputPort.IsValid() || !OtherPinnedOutputPort.IsValid())
		{
			return false;
		}
		const bool bSameProtocol = PinnedOutputPort->GetProtocol() == OtherPinnedOutputPort->GetProtocol();
		if (!bSameProtocol)
		{
			return false;
		}

		const bool bSameDeviceAddress = PinnedOutputPort->GetDeviceAddress() == OtherPinnedOutputPort->GetDeviceAddress();
		const TArray<FString> DestinationAddresses = PinnedOutputPort->GetDestinationAddresses();
		const bool bSameDestination = Algo::FindByPredicate(OtherPinnedOutputPort->GetDestinationAddresses(), [&DestinationAddresses](const FString& OtherDestinationAddress)
			{
				return DestinationAddresses.Contains(OtherDestinationAddress);
			}) != nullptr;

		if (!bSameDeviceAddress && !bSameDestination)
		{
			return false;
		}

		const bool bConflictingChannels = Algo::FindByPredicate(ChannelToValueMap, [Other](const TTuple<int32, uint8> ChannelToValuePair)
			{
				return Other.ChannelToValueMap.Contains(ChannelToValuePair.Key);
			}) != nullptr;

		return bConflictingChannels;
	}

	FDMXConflictMonitorUserSession::~FDMXConflictMonitorUserSession()
	{
		FDMXConflictMonitor::RemoveUser(UserName);
	}

	FDMXConflictMonitorUserSession::FDMXConflictMonitorUserSession(FName InUserName)
		: UserName(InUserName)
	{}

	TArray<FName> FDMXConflictMonitor::UserNames;
	TSharedPtr<FDMXConflictMonitor> FDMXConflictMonitor::Instance;

	FDMXConflictMonitor::~FDMXConflictMonitor()
	{
		ensureMsgf(UserNames.IsEmpty(), TEXT("Not all user objects left the monitor before it was shut down."));
	}

	TSharedRef<FDMXConflictMonitorUserSession> FDMXConflictMonitor::Join(const FName& UserName)
	{
		if (UserNames.IsEmpty())
		{
			Instance = MakeShared<FDMXConflictMonitor>();
		}
		return MakeShared<FDMXConflictMonitorUserSession>(UserName);
	}

	bool FDMXConflictMonitor::IsEnabled()
	{
		return Instance.IsValid();
	}

	FDMXConflictMonitor* FDMXConflictMonitor::Get()
	{
		return Instance.Get();
	}

	TMap<FName, TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>> FDMXConflictMonitor::GetOutboundConflictsSynchronous() const
	{
		const FScopeLock Lock(&MutexAccessConflicts);
		return Conflicts;
	}

	void FDMXConflictMonitor::TraceUser(const FMinimalName& User)
	{
		TraceStack.Add(User);
	}

	void FDMXConflictMonitor::PopTrace(const FMinimalName& User)
	{
		checkf(User == TraceStack.Last(), TEXT("Detected nested trace, parent is runing out of scope before child. Only nested scopes are supported."));

		TraceStack.Pop();
	}

	void FDMXConflictMonitor::MonitorOutboundDMX(const TSharedRef<FDMXOutputPort>& InOutputPort, int32 InLocalUniverseID, const TMap<int32, uint8>& InChannelToValueMap)
	{
		SCOPE_CYCLE_COUNTER(STAT_DMXConflictMonitor)

		check(IsInGameThread());
		const double StartTime = FPlatformTime::Seconds();

		// Only do work when the frame switched
		if (FrameNumber != GFrameNumber)
		{
			UE::Tasks::TTask<void> Task;
			
			// Drop frames if there's too much data
			if (!Task.IsValid() || Task.IsCompleted()) 
			{
				Task = UE::Tasks::Launch(TEXT("FDMXConflictMonitor::MonitorOutboundDMXGameThread"), [DataTaskThread = MonitoredOutboundData, SharedThis = AsShared(), this]
					{
						TMap<FName, TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>> ConflictsTaskThread;

						for (const TSharedRef<FDMXMonitoredOutboundDMXData>& Data : DataTaskThread)
						{
							const TSharedRef<FDMXMonitoredOutboundDMXData>* ConflictPtr = Algo::FindByPredicate(DataTaskThread, [Data, &DataTaskThread](const TSharedRef<FDMXMonitoredOutboundDMXData>& Other)
								{
									// Avoid self, and duplicates where { snapshot, other } == { other, snapshot }
									const bool bSelfOrDuplicate = Data->Trace.Compare(Other->Trace) >= 0;
									return
										!bSelfOrDuplicate &&
										Other->ConflictsWith(*Data);
								});

							if (ConflictPtr)
							{
								TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>& ConflictsWithData =
									ConflictsTaskThread.FindOrAdd(Data->Trace, TArray<TSharedRef<FDMXMonitoredOutboundDMXData>>({ Data }));
								ConflictsWithData.Add(*ConflictPtr);
							}
						}

						// Copy to game thread
						const FScopeLock Lock(&MutexAccessConflicts);
						Conflicts = ConflictsTaskThread;
					},
					UE::Tasks::ETaskPriority::BackgroundLow
				);
			}
			
			MonitoredOutboundData.Reset();
		}

		if (!TraceStack.IsEmpty())
		{
			FString CombinedTrace;
			for (const FMinimalName& Trace : TraceStack)
			{
				CombinedTrace += FName(Trace).ToString();
				if (&Trace != &TraceStack.Last())
				{
					CombinedTrace += TEXT(",");
				}
			}
			MonitoredOutboundData.Add(MakeShared<FDMXMonitoredOutboundDMXData>(InOutputPort, InLocalUniverseID, InChannelToValueMap, *CombinedTrace));
		}

		FrameNumber = GFrameNumber;

		const double EndTime = FPlatformTime::Seconds();
		TimeGameThread = (EndTime - StartTime) * 1000.0;
	}

	void FDMXConflictMonitor::RemoveUser(const FName& UserName)
	{
		UserNames.Remove(UserName);
		if (UserNames.IsEmpty() &&
			ensureMsgf(Instance.IsValid(), TEXT("Unexpected tried to remove a user from conflict monitor, but the conflict monitor instance is invalid.")))
		{
			Instance->MonitoredOutboundData.Reset();
			Instance.Reset();
		}
	}
}

#endif // WITH_EDITOR
