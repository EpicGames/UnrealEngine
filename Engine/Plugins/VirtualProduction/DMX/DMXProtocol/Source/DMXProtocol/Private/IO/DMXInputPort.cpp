// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/DMXInputPort.h"

#include "Async/Async.h"
#include "DMXProtocolSettings.h"
#include "DMXProtocolUtils.h"
#include "DMXStats.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXInputPortConfig.h"
#include "IO/DMXPortManager.h"
#include "IO/DMXRawListener.h"

#include "HAL/IConsoleManager.h"

DECLARE_CYCLE_STAT(TEXT("Input Port Tick"), STAT_DMXInputPortTick, STATGROUP_DMX);


#define LOCTEXT_NAMESPACE "DMXInputPort"

/** Helper to override a member variable of an Input Port */
#define DMX_OVERRIDE_INPUTPORT_VAR(MemberName, PortName, Value) \
{ \
	const FDMXInputPortSharedPtr OverRideInputPortMacro_InputPort = UE::DMX::DMXInputPort::Private::ConsoleCommands::FindInputPortByName(PortName); \
	if (OverRideInputPortMacro_InputPort.IsValid()) \
	{ \
		const FDMXInputPortConfig OldInputPortConfig = OverRideInputPortMacro_InputPort->MakeInputPortConfig(); \
		FDMXInputPortConfigParams InputPortConfigParams = FDMXInputPortConfigParams(OldInputPortConfig); \
		InputPortConfigParams.MemberName = Value; \
		FDMXInputPortConfig NewInputPortConfig(OverRideInputPortMacro_InputPort->GetPortGuid(), InputPortConfigParams); \
		constexpr bool bForceUpdateRegistrationWithProtocol = true; \
		OverRideInputPortMacro_InputPort->UpdateFromConfig(NewInputPortConfig, bForceUpdateRegistrationWithProtocol); \
	} \
}

namespace UE::DMX::DMXInputPort::Private::ConsoleCommands
{
	void LogInputPortConfiguration(const FDMXInputPortSharedPtr InputPort)
	{
		if (InputPort.IsValid())
		{
			UE_LOGF(LogDMXProtocol, Log, "================================================");
			UE_LOGF(LogDMXProtocol, Log, "Configuration of Input Port '%ls':", *InputPort->GetPortName());
			UE_LOGF(LogDMXProtocol, Log, " ");

			UE_LOGF(LogDMXProtocol, Log, "Port Name: %ls", *InputPort->GetPortName());

			FDMXInputPortConfig PortConfig = InputPort->MakeInputPortConfig();
			UE_LOGF(LogDMXProtocol, Log, "Protocol: %ls", *PortConfig.GetProtocolName().ToString());
			UE_LOGF(LogDMXProtocol, Log, "Communication Type: %i", static_cast<int32>(PortConfig.GetCommunicationType()));
			UE_LOGF(LogDMXProtocol, Log, "Device Address: %ls", *PortConfig.GetDeviceAddress());

			UE_LOGF(LogDMXProtocol, Log, "Local Universe Start: %i", PortConfig.GetLocalUniverseStart());
			UE_LOGF(LogDMXProtocol, Log, "Num Universes: %i", PortConfig.GetNumUniverses());
			UE_LOGF(LogDMXProtocol, Log, "Extern Universe Start: %i", PortConfig.GetExternUniverseStart());
			UE_LOGF(LogDMXProtocol, Log, "Priority Strategy: %i", static_cast<int32>(PortConfig.GetPortPriorityStrategy()));
			UE_LOGF(LogDMXProtocol, Log, "Priority:	%i", PortConfig.GetPriority());

			UE_LOGF(LogDMXProtocol, Log, "================================================");
		}
	}

	/** Finds an Input port by its name, optionally logging when it can't be found */
	static const FDMXInputPortSharedPtr FindInputPortByName(const FString& PortName, bool bPrintToLogIfNotFound = true)
	{
		const FDMXInputPortSharedRef* InputPortPtr = FDMXPortManager::Get().GetInputPorts().FindByPredicate([PortName](const FDMXInputPortSharedPtr& InputPort)
			{
				return InputPort->GetPortName() == PortName;
			});

		if (InputPortPtr)
		{
			return *InputPortPtr;
		}
		else if (bPrintToLogIfNotFound)
		{
			if (PortName.IsEmpty())
			{
				UE_LOGF(LogDMXProtocol, Warning, "No Input Port specified for console command. First argument needs to be a valid Port Name.");
			}
			else
			{
				UE_LOGF(LogDMXProtocol, Warning, "Could not find Input Port '%ls'. Available Input Ports and their names can be found in in Project settings -> Plugins -> DMX.", *PortName);
			}
		}

		return nullptr;
	}

	/** Tests the console command, returns true if they yield a valid port. Logs issues otherwise. Handles ? commands. */
	static bool VerifyConsoleCommandArguments(const FString& ConsoleCommand, int32 MinNumExpectedArgs, const TArray<FString>& Args)
	{
		const FString PortName = Args.Num() > 0 ? Args[0] : FString();
		const FDMXInputPortSharedPtr InputPort = FindInputPortByName(PortName);

		// Log the port if the there is only one arg or the second arg is '?' (e.g. 'DMX.SetInputPortProtocol InputPortA' or 'DMX.SetInputPortProtocol InputPortA ?')
		const bool bOnlyPortSpecified = Args.Num() == 1;
		const bool bSecondArgIsQuestionmark = (Args.Num() == 2 && Args[1] == TEXT("?"));
		if (InputPort.IsValid() && (bOnlyPortSpecified || bSecondArgIsQuestionmark))
		{
			LogInputPortConfiguration(InputPort);
			
			return false;
		}
		else if (!InputPort.IsValid())
		{
			// Log additional help if there were not enough arguments or arguments were invalid
			UE_LOGF(LogDMXProtocol, Warning, "Console Command %ls failed. Input Port with Name '%ls' does not exist.", *ConsoleCommand, *PortName);
			UE_LOGF(LogDMXProtocol, Warning, "Ports and their Names can be found in Project Settings -> Plugins -> DMX.");
			
			return false;
		}
		else if (MinNumExpectedArgs > Args.Num())
		{
			UE_LOGF(LogDMXProtocol, Warning, "Console Command %ls failed. Invalid arguments specified.", *ConsoleCommand);
			UE_LOGF(LogDMXProtocol, Warning, "Use '%ls ?' for help. Use '%ls PortName ?' to print the current configuration to logs.", *ConsoleCommand, *ConsoleCommand);
			
			return false;
		}


		return true;
	}

	static FAutoConsoleCommand GDMXLogAllInputPortConfigurationsCommand(
		TEXT("DMX.LogAllInputPortConfigurations"), 
		TEXT("Logs all Input Port configurations"), 
		FConsoleCommandDelegate::CreateStatic(
			[]()
			{
				for (const FDMXInputPortSharedRef& InputPort : FDMXPortManager::Get().GetInputPorts())
				{
					LogInputPortConfiguration(InputPort);
				}
			})
	);

	static FAutoConsoleCommand GDMXSetInputPortProtocolCommand(
		TEXT("DMX.SetInputPortProtocol"),
		TEXT("DMX.SetInputPortProtocol [PortName][ProtocolName]. Sets the protocol used by the input port. Example: DMX.SetInputPortProtocol MyInputPort sACN"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetInputPortProtocol"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FName ProtocolNameValue = FName(Args[1]);

				if (IDMXProtocol::GetProtocolNames().Contains(ProtocolNameValue))
				{
					DMX_OVERRIDE_INPUTPORT_VAR(ProtocolName, PortName, ProtocolNameValue);
				}
				else
				{
					UE_LOGF(LogDMXProtocol, Warning, "Console command DMX.SetInputPortProtocol failed. Protocol '%ls' does not exist or is not loaded. Available protocols are: ", *ProtocolNameValue.ToString());
					for (const FName& ProtocolName : IDMXProtocol::GetProtocolNames())
					{
						UE_LOGF(LogDMXProtocol, Warning, "%ls", *ProtocolName.ToString());
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetInputPortDeviceAddressCommand(
		TEXT("DMX.SetInputPortDeviceAddress"),
		TEXT("DMX.SetInputPortDeviceAddress [PortName][DeviceAddress]. Sets the Device Address of an input port, usually the network interface card IP address. Example: DMX.SetInputPortDeviceAddress MyInputPort 123.45.67.89"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetInputPortDeviceAddress"), MinNumExpectedArgs, Args))
				{							
					// For this command, additionally log the available Local Network Interface Card IP addresses
					UE_LOGF(LogDMXProtocol, Log, "================================================");
					UE_LOGF(LogDMXProtocol, Log, "Logging available Local Network Interface Card IP Addresses:");
					const TArray<TSharedPtr<FString>> AvailableIPAddresses = FDMXProtocolUtils::GetLocalNetworkInterfaceCardIPs();
					for (int32 IPAddressIndex = 0; IPAddressIndex < AvailableIPAddresses.Num(); IPAddressIndex++)
					{
						UE_LOGF(LogDMXProtocol, Log, "%i: %ls", IPAddressIndex, *(*AvailableIPAddresses[IPAddressIndex]));
					}
					UE_LOGF(LogDMXProtocol, Log, "================================================");

					return;
				}

				const FString& PortName = Args[0];
				const FString& DeviceAddressValue = Args[1];

				DMX_OVERRIDE_INPUTPORT_VAR(DeviceAddress, PortName, DeviceAddressValue);
			})
	);

	static FAutoConsoleCommand GDMXSetInputPortLocalUniverseStartCommand(
		TEXT("DMX.SetInputPortLocalUniverseStart"),
		TEXT("DMX.SetInputPortLocalUniverseStart [PortName][Universe]. Sets the local universe start of the input port. Example: DMX.SetInputPortLocalUniverseStart MyInputPort 5"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetInputPortLocalUniverseStart"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FDMXInputPortSharedPtr InputPort = FindInputPortByName(PortName);

				if (InputPort.IsValid())
				{
					if (const IDMXProtocolPtr& Protocol = InputPort->GetProtocol())
					{
						const FString& LocalUniverseStartValueString = Args[1];
						int32 LocalUniverseStartValue;
						if (LexTryParseString<int32>(LocalUniverseStartValue, *LocalUniverseStartValueString))
						{
							const int32 NumUniverses = InputPort->GetLocalUniverseEnd() - InputPort->GetLocalUniverseStart() + 1;

							const int32 ExternUniverseStart = InputPort->ConvertLocalToExternUniverseID(LocalUniverseStartValue);
							const int32 ExternUniverseEnd = InputPort->ConvertLocalToExternUniverseID(LocalUniverseStartValue + NumUniverses - 1);

							if (Protocol->IsValidUniverseID(ExternUniverseStart) &&
								Protocol->IsValidUniverseID(ExternUniverseEnd))
							{
								DMX_OVERRIDE_INPUTPORT_VAR(LocalUniverseStart, PortName, LocalUniverseStartValue);
							}
							else
							{
								UE_LOGF(LogDMXProtocol, Warning, "Console command DMX.SetInputPortLocalUniverseStart failed. Local Universe Start '%ls' along with Num Universes '%i' results in a Universe range that is not supported by the Protocol of the Port.", *LocalUniverseStartValueString, NumUniverses);
							}
						}
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetInputPortNumUniversesCommand(
		TEXT("DMX.SetInputPortNumUniverses"),
		TEXT("DMX.SetInputPortNumUniverses [PortName][Universe]. Sets the num universes of the input port. Example: DMX.SetInputPortNumUniverses MyInputPort 10"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetInputPortNumUniverses"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FDMXInputPortSharedPtr InputPort = FindInputPortByName(PortName);

				if (InputPort.IsValid())
				{
					if (const IDMXProtocolPtr& Protocol = InputPort->GetProtocol())
					{
						const FString& NumUniversesValueString = Args[1];
						int32 NumUniversesValue;
						if (LexTryParseString<int32>(NumUniversesValue, *NumUniversesValueString))
						{
							const int32 LocalUniverseStart = InputPort->GetLocalUniverseStart();

							const int32 ExternUniverseStart = InputPort->ConvertLocalToExternUniverseID(LocalUniverseStart);
							const int32 ExternUniverseEnd = ExternUniverseStart + NumUniversesValue - 1;

							if (NumUniversesValue > 0 &&
								Protocol->IsValidUniverseID(ExternUniverseStart) &&
								Protocol->IsValidUniverseID(ExternUniverseEnd))
							{
								DMX_OVERRIDE_INPUTPORT_VAR(NumUniverses, PortName, NumUniversesValue);
							}
							else
							{
								UE_LOGF(LogDMXProtocol, Warning, "Console command DMX.SetInputPortNumUniverses failed. Local Universe Start '%i' along with Num Universes '%ls' results in a Universe range that is not supported by the Protocol of the Port.", LocalUniverseStart, *NumUniversesValueString);
							}
						}
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetInputPortExternUniverseStartCommand(
		TEXT("DMX.SetInputPortExternUniverseStart"),
		TEXT("DMX.SetInputPortExternUniverseStart [PortName][Universe]. Sets the extern universe start of the input port. Example: DMX.SetInputPortExternUniverseStart MyInputPort 7"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetInputPortExternUniverseStart"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FDMXInputPortSharedPtr InputPort = FindInputPortByName(PortName);

				if (InputPort.IsValid())
				{
					const FString& ExternUniverseStartValueString = Args[1];
					int32 ExternUniverseStartValue;
					if (LexTryParseString<int32>(ExternUniverseStartValue, *ExternUniverseStartValueString))
					{
						if (const IDMXProtocolPtr& Protocol = InputPort->GetProtocol())
						{
							const int32 NumUniverses = InputPort->GetLocalUniverseEnd() - InputPort->GetLocalUniverseStart() + 1;
							const int32 ExternUniverseEnd = ExternUniverseStartValue + NumUniverses - 1;

							if (Protocol->IsValidUniverseID(ExternUniverseStartValue) &&
								Protocol->IsValidUniverseID(ExternUniverseEnd))
							{
								DMX_OVERRIDE_INPUTPORT_VAR(ExternUniverseStart, PortName, ExternUniverseStartValue);
							}
							else
							{
								UE_LOGF(LogDMXProtocol, Warning, "Console command DMX.SetInputPortExternUniverseStart failed. Extern Universe Start '%ls' along with Num Universes '%i' results in a Universe range that is not supported by the Protocol of the Port.", *ExternUniverseStartValueString, NumUniverses);
							}
						}
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetInputPortPriorityStrategyCommand(
		TEXT("DMX.SetInputPortPriorityStrategy"),
		TEXT("DMX.SetInputPortPriorityStrategy [PortName][PriorityStrategy (0 = None, 1 = Lowest, 2 = LowerThan, 3 = Equal, 4 = Higher Than, 5 = Highest)]. Example: DMX.SetInputPortPriorityStrategy MyInputPort 1"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetInputPortPriorityStrategy"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FString& PriorityStrategyValueString = Args[1];
				uint8 PriorityStrategyValue;
				if (LexTryParseString<uint8>(PriorityStrategyValue, *PriorityStrategyValueString))
				{
					const EDMXPortPriorityStrategy PriorityStrategyEnumValue = static_cast<EDMXPortPriorityStrategy>(PriorityStrategyValue);

					if (PriorityStrategyEnumValue == EDMXPortPriorityStrategy::None ||
						PriorityStrategyEnumValue == EDMXPortPriorityStrategy::Lowest ||
						PriorityStrategyEnumValue == EDMXPortPriorityStrategy::LowerThan ||
						PriorityStrategyEnumValue == EDMXPortPriorityStrategy::Equal ||
						PriorityStrategyEnumValue == EDMXPortPriorityStrategy::HigherThan ||
						PriorityStrategyEnumValue == EDMXPortPriorityStrategy::Highest)
					{
						DMX_OVERRIDE_INPUTPORT_VAR(PriorityStrategy, PortName, PriorityStrategyEnumValue);
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetInputPortPriorityCommand(
		TEXT("DMX.SetInputPortPriority"),
		TEXT("DMX.SetInputPortPriority [PortName][Priority]. Sets the priority of the input port. Example: DMX.SetInputPortPriority MyInputPort 100"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetInputPortPriority"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FString& PriorityValueString = Args[1];
				int32 PriorityValue;
				if (LexTryParseString<int32>(PriorityValue, *PriorityValueString))
				{
					DMX_OVERRIDE_INPUTPORT_VAR(Priority, PortName, PriorityValue);
				}
			})
	);

	static FAutoConsoleCommand GDMXResetInputPortToProjectSettings(
		TEXT("DMX.ResetInputPortToProjectSettings"),
		TEXT("DMX.ResetInputPortToProjectSettings [PortName]. Resets the input port to how it is defined in project settings. Example: DMX.ResetInputPortToProjectSettings MyInputPort"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 1;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.ResetInputPortToProjectSettings"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];

				const FDMXInputPortSharedPtr InputPort = FindInputPortByName(PortName);
				if(InputPort.IsValid())
				{
					const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
					if (ProtocolSettings)
					{
						const FDMXInputPortConfig* PortConfigPtr = ProtocolSettings->InputPortConfigs.FindByPredicate([InputPort](const FDMXInputPortConfig& InputPortConfig)
							{
								return InputPortConfig.GetPortGuid() == InputPort->GetPortGuid();
							});

						if (PortConfigPtr)
						{
							FDMXInputPortConfig PortConfig = *PortConfigPtr;
							InputPort->UpdateFromConfig(PortConfig);
						}
					}
				}
			})
	);
}
#undef DMX_OVERRIDE_INPUTPORT_VAR

FDMXInputPortSharedRef FDMXInputPort::CreateFromConfig(FDMXInputPortConfig& InputPortConfig)
{
	// Port Configs are expected to have a valid guid always
	check(InputPortConfig.GetPortGuid().IsValid());

	FDMXInputPortSharedRef NewInputPort = MakeShared<FDMXInputPort, ESPMode::ThreadSafe>();

	NewInputPort->PortGuid = InputPortConfig.GetPortGuid();

	UDMXProtocolSettings* Settings = GetMutableDefault<UDMXProtocolSettings>();
	check(Settings);

	NewInputPort->bReceiveDMXEnabled = Settings->IsReceiveDMXEnabled();

	// Bind to receive dmx changes
	Settings->GetOnSetReceiveDMXEnabled().AddThreadSafeSP(NewInputPort, &FDMXInputPort::OnSetReceiveDMXEnabled);

	NewInputPort->UpdateFromConfig(InputPortConfig);

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Created input port %ls", *NewInputPort->PortName);

	return NewInputPort;
}

FDMXInputPort::~FDMXInputPort()
{
	// Port needs be unregistered before destruction
	check(!bRegistered);

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Destroyed input port %ls", *PortName);
}

FDMXInputPortConfig FDMXInputPort::MakeInputPortConfig() const
{
	FDMXInputPortConfigParams Params;
	Params.PortName = PortName;
	Params.ProtocolName = Protocol.IsValid() ? Protocol->GetProtocolName() : NAME_None;
	Params.bAutoCompleteDeviceAddressEnabled = bAutoCompleteDeviceAddressEnabled;
	Params.AutoCompleteDeviceAddress = AutoCompleteDeviceAddress;
	Params.CommunicationType = CommunicationType;
	Params.DeviceAddress = DeviceAddress;
	Params.LocalUniverseStart = LocalUniverseStart;
	Params.NumUniverses = NumUniverses;
	Params.ExternUniverseStart = ExternUniverseStart;
	Params.PriorityStrategy = PriorityStrategy;
	Params.Priority = Priority;

	return FDMXInputPortConfig(PortGuid, Params);
}

void FDMXInputPort::UpdateFromConfig(FDMXInputPortConfig& InOutInputPortConfig, bool bForceUpdateRegistrationWithProtocol)
{
	// Need a valid config for the port
	InOutInputPortConfig.MakeValid();

	// Avoid further changes to the config
	const FDMXInputPortConfig& InputPortConfig = InOutInputPortConfig;

	// Can only use configs that correspond to project settings
	const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
	const bool bConfigIsInProjectSettings = ProtocolSettings->InputPortConfigs.ContainsByPredicate([&InputPortConfig](const FDMXInputPortConfig& Other) {
		return InputPortConfig.GetPortGuid() == Other.GetPortGuid();
		});
	ensureAlwaysMsgf(bConfigIsInProjectSettings, TEXT("Can only use configs with a guid that corresponds to a config in project settings"));

	// Find if the port needs update its registration with the protocol
	const bool bNeedsUpdateRegistration = [this, &InputPortConfig, bForceUpdateRegistrationWithProtocol]()
	{
		if (bForceUpdateRegistrationWithProtocol)
		{
			return true;
		}

		if (IsRegistered() != bReceiveDMXEnabled)
		{
			return true;
		}

		if (IsRegistered() && FDMXPortManager::Get().AreProtocolsSuspended())
		{
			return true;
		}

		FName ProtocolName = Protocol.IsValid() ? Protocol->GetProtocolName() : NAME_None;

		if (ProtocolName == InputPortConfig.GetProtocolName() &&
			bAutoCompleteDeviceAddressEnabled == InputPortConfig.IsAutoCompleteDeviceAddressEnabled() &&
			AutoCompleteDeviceAddress == InputPortConfig.GetAutoCompleteDeviceAddress() &&
			DeviceAddress == InputPortConfig.GetDeviceAddress() &&
			CommunicationType == InputPortConfig.GetCommunicationType())
		{
			return false;
		}

		return true;
	}();

	// Unregister the port if required before the new protocol is set
	if (bNeedsUpdateRegistration)
	{
		Unregister();
	}

	Protocol = IDMXProtocol::Get(InputPortConfig.GetProtocolName());

	// Copy properties from the config into the base class
	PortGuid = InputPortConfig.GetPortGuid();
	CommunicationType = InputPortConfig.GetCommunicationType();
	bAutoCompleteDeviceAddressEnabled = InputPortConfig.IsAutoCompleteDeviceAddressEnabled();
	AutoCompleteDeviceAddress = InputPortConfig.GetAutoCompleteDeviceAddress();
	DeviceAddress = InputPortConfig.GetDeviceAddress();
	ExternUniverseStart = InputPortConfig.GetExternUniverseStart();
	LocalUniverseStart = InputPortConfig.GetLocalUniverseStart();
	NumUniverses = InputPortConfig.GetNumUniverses();
	PortName = InputPortConfig.GetPortName();
	PriorityStrategy = InputPortConfig.GetPortPriorityStrategy();
	Priority = InputPortConfig.GetPriority();

	// Only update on the next tick to give the OS time to fully release previous sockets
	if (bPendingRegistration)
	{
		return;
	}
	bPendingRegistration = true;

	AsyncTask(ENamedThreads::GameThread, [bNeedsUpdateRegistration, WeakThis = AsWeak(), this]()
		{
			if (!WeakThis.IsValid())
			{
				return;
			}

			// Re-register the port if required
			if (bNeedsUpdateRegistration)
			{
				Register();
			}

			bPendingRegistration = false;

			OnPortUpdated.Broadcast();
		});
}

bool FDMXInputPort::IsRegistered() const
{
	return bRegistered;
}

const FGuid& FDMXInputPort::GetPortGuid() const
{
	check(PortGuid.IsValid());
	return PortGuid;
}

bool FDMXInputPort::CheckPriority(const int32 InPriority)
{
	if (InPriority > HighestReceivedPriority)
	{
		HighestReceivedPriority = InPriority;
	}

	if (InPriority < LowestReceivedPriority)
	{
		LowestReceivedPriority = InPriority;
	}

	switch (PriorityStrategy)
	{
	case(EDMXPortPriorityStrategy::None):
		return true;
	case(EDMXPortPriorityStrategy::HigherThan):
		return InPriority > Priority;
	case(EDMXPortPriorityStrategy::Equal):
		return InPriority == Priority;
	case(EDMXPortPriorityStrategy::LowerThan):
		return InPriority < Priority;
	case(EDMXPortPriorityStrategy::Highest):
		return InPriority >= HighestReceivedPriority;
	case(EDMXPortPriorityStrategy::Lowest):
		return InPriority <= LowestReceivedPriority;
	default:
		break;
	}

	return false;
}

void FDMXInputPort::AddRawListener(TSharedRef<FDMXRawListener> InRawListener)
{
	const FScopeLock LockRawListeners(&AccessRawListenersMutex);

	check(!RawListeners.Contains(InRawListener));

	RawListeners.Add(InRawListener);
}

void FDMXInputPort::RemoveRawListener(TSharedRef<FDMXRawListener> InRawListenerToRemove)
{
	const FScopeLock LockRawListeners(&AccessRawListenersMutex);

	RawListeners.Remove(InRawListenerToRemove);
}

bool FDMXInputPort::Register()
{
	if (Protocol.IsValid() && IsValidPortSlow() && bReceiveDMXEnabled && !FDMXPortManager::Get().AreProtocolsSuspended())
	{
		bRegistered = Protocol->RegisterInputPort(SharedThis(this));

		return bRegistered;
	}

	return false;
}

void FDMXInputPort::Unregister()
{
	if (bRegistered)
	{
		if (Protocol.IsValid())
		{
			Protocol->UnregisterInputPort(SharedThis(this));
		}

		bRegistered = false;
	}
}

void FDMXInputPort::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_DMXInputPortTick);

	// Tick universe Inputs with latest signals on tick
	FDMXSignalSharedPtr Signal;

	if (bUseDefaultInputQueue)
	{
		while (DefaultInputQueue.Dequeue(Signal))
		{
			// No need to filter extern universe, we already did that when enqueing
			ExternUniverseToLatestSignalMap.FindOrAdd(Signal->ExternUniverseID) = Signal;
		}
	}
}

ETickableTickType FDMXInputPort::GetTickableTickType() const
{
	return ETickableTickType::Always;
}

TStatId FDMXInputPort::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDMXInputPort, STATGROUP_Tickables);
}

void FDMXInputPort::ClearBuffers()
{
#if UE_BUILD_DEBUG
	// Needs be called from the game thread, to maintain thread safety with ExternUniverseToLatestSignalMap
	check(IsInGameThread());
#endif // UE_BUILD_DEBUG

	AccessRawListenersMutex.Lock();
	for (const TSharedRef<FDMXRawListener>& RawListener : RawListeners)
	{
		RawListener->ClearBuffer();
	}
	AccessRawListenersMutex.Unlock();

	DefaultInputQueue.Empty();
	ExternUniverseToLatestSignalMap.Reset();
}

void FDMXInputPort::InputDMXSignal(const FDMXSignalSharedRef& DMXSignal)
{
	if (IsReceiveDMXEnabled())
	{
		int32 ExternUniverseID = DMXSignal->ExternUniverseID;
		if (IsExternUniverseInPortRange(ExternUniverseID))
		{
			AccessRawListenersMutex.Lock();
			for (const TSharedRef<FDMXRawListener>& RawListener : RawListeners)
			{
				RawListener->EnqueueSignal(this, DMXSignal);
			}
			AccessRawListenersMutex.Unlock();

			if (bUseDefaultInputQueue)
			{
				DefaultInputQueue.Enqueue(DMXSignal);
			}
		}
	}
}

void FDMXInputPort::SingleProducerInputDMXSignal(const FDMXSignalSharedRef& DMXSignal)
{
	// DERECATED 5.0

	if (IsReceiveDMXEnabled())
	{
		int32 ExternUniverseID = DMXSignal->ExternUniverseID;
		if (IsExternUniverseInPortRange(ExternUniverseID))
		{
			AccessRawListenersMutex.Lock();
			for (const TSharedRef<FDMXRawListener>& RawListener : RawListeners)
			{
				RawListener->EnqueueSignal(this, DMXSignal);
			}
			AccessRawListenersMutex.Unlock();

			if (bUseDefaultInputQueue)
			{
				DefaultInputQueue.Enqueue(DMXSignal);
			}
		}
	}
}

bool FDMXInputPort::GameThreadGetDMXSignal(int32 LocalUniverseID, FDMXSignalSharedPtr& OutDMXSignal)
{
#if UE_BUILD_DEBUG
	check(IsInGameThread());
#endif // UE_BUILD_DEBUG

	int32 ExternUniverseID = ConvertLocalToExternUniverseID(LocalUniverseID);

	const FDMXSignalSharedPtr* SignalPtr = ExternUniverseToLatestSignalMap.Find(ExternUniverseID);
	if (SignalPtr)
	{
		OutDMXSignal = *SignalPtr;
		return true;
	}

	return false;
}

const TMap<int32, FDMXSignalSharedPtr>& FDMXInputPort::GameThreadGetAllDMXSignals() const
{
#if UE_BUILD_DEBUG
	check(IsInGameThread());
#endif // UE_BUILD_DEBUG

	return ExternUniverseToLatestSignalMap;
}

bool FDMXInputPort::GameThreadGetDMXSignalFromRemoteUniverse(FDMXSignalSharedPtr& OutDMXSignal, int32 RemoteUniverseID, bool bEvenIfNotLoopbackToEngine)
{
	// DEPRECATED 4.27
#if UE_BUILD_DEBUG
	check(IsInGameThread());
#endif // UE_BUILD_DEBUG

	const FDMXSignalSharedPtr* SignalPtr = ExternUniverseToLatestSignalMap.Find(RemoteUniverseID);
	if (SignalPtr)
	{
		OutDMXSignal = *SignalPtr;
		return true;
	}

	return false;
}

void FDMXInputPort::GameThreadInjectDMXSignal(const FDMXSignalSharedRef& DMXSignal)
{
	ensureMsgf(!bUseDefaultInputQueue || !bReceiveDMXEnabled || FDMXPortManager::Get().AreProtocolsSuspended(), TEXT("Potential conflicts between injected and received signals, please revise the implementation."));

	int32 ExternUniverseID = DMXSignal->ExternUniverseID;
	if (IsExternUniverseInPortRange(ExternUniverseID))
	{
		ExternUniverseToLatestSignalMap.FindOrAdd(ExternUniverseID, DMXSignal) = DMXSignal;
	}
}

void FDMXInputPort::SetUseDefaultQueue(bool bUse)
{
	bUseDefaultInputQueue = bUse;
}

void FDMXInputPort::OnSetReceiveDMXEnabled(bool bEnabled)
{
	bReceiveDMXEnabled = bEnabled;

	FDMXInputPortConfig InputPortConfig = MakeInputPortConfig();
	UpdateFromConfig(InputPortConfig);
}

#undef LOCTEXT_NAMESPACE
