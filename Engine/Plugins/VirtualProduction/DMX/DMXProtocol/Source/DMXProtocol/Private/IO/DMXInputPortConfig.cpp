// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/DMXInputPortConfig.h"

#include "DMXProtocolSettings.h"
#include "DMXProtocolUtils.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXPortManager.h"

#include "Misc/Guid.h"


FDMXInputPortConfigParams::FDMXInputPortConfigParams(const FDMXInputPortConfig& InputPortConfig)
	: PortName(InputPortConfig.GetPortName())
	, ProtocolName(InputPortConfig.GetProtocolName())
	, CommunicationType(InputPortConfig.GetCommunicationType())
	, bAutoCompleteDeviceAddressEnabled(InputPortConfig.IsAutoCompleteDeviceAddressEnabled())
	, AutoCompleteDeviceAddress(InputPortConfig.GetAutoCompleteDeviceAddress())
	, DeviceAddress(InputPortConfig.GetDeviceAddress())
	, LocalUniverseStart(InputPortConfig.GetLocalUniverseStart())
	, NumUniverses(InputPortConfig.GetNumUniverses())
	, ExternUniverseStart(InputPortConfig.GetExternUniverseStart())
	, PriorityStrategy(InputPortConfig.GetPortPriorityStrategy())
	, Priority(InputPortConfig.GetPriority())
{}


FDMXInputPortConfig::FDMXInputPortConfig()
	: PortGuid(FGuid::NewGuid())
{}

FDMXInputPortConfig::FDMXInputPortConfig(const FGuid& InPortGuid)
	: PortGuid(InPortGuid)
{
	// Cannot create port configs before the protocol module is up (it is required to sanetize protocol names).
	check(FModuleManager::Get().IsModuleLoaded("DMXProtocol"));
	check(PortGuid.IsValid());

	MakeValid();
}

FDMXInputPortConfig::FDMXInputPortConfig(const FGuid& InPortGuid, const FDMXInputPortConfigParams& InitializationData)	
	: PortName(InitializationData.PortName)
	, ProtocolName(InitializationData.ProtocolName)
	, CommunicationType(InitializationData.CommunicationType)
	, bAutoCompleteDeviceAddressEnabled(InitializationData.bAutoCompleteDeviceAddressEnabled)
	, AutoCompleteDeviceAddress(InitializationData.AutoCompleteDeviceAddress)
	, DeviceAddress(InitializationData.DeviceAddress)
	, LocalUniverseStart(InitializationData.LocalUniverseStart)
	, NumUniverses(InitializationData.NumUniverses)
	, ExternUniverseStart(InitializationData.ExternUniverseStart)
	, PriorityStrategy(InitializationData.PriorityStrategy)
	, Priority(InitializationData.Priority)
	, PortGuid(InPortGuid)
{
	// Cannot create port configs before the protocol module is up (it is required to sanetize protocol names).
	check(FModuleManager::Get().IsModuleLoaded("DMXProtocol"));
	check(PortGuid.IsValid());
	check(!ProtocolName.IsNone())

	MakeValid();
}

void FDMXInputPortConfig::MakeValid()
{
	if (!ensureAlwaysMsgf(PortGuid.IsValid(), TEXT("Invalid GUID for Input Port %s. Generating a new one. Blueprint nodes referencing the port will no longer be functional."), *PortName))
	{
		PortGuid = FGuid::NewGuid();
	}

	IDMXProtocolPtr Protocol = IDMXProtocol::Get(ProtocolName);

	// Try to restore the protocol if it is not valid.
	// Allow NAME_None as an option if no protocol should be loaded (e.g. in projects that play a dmx show from sequencer only).
	if (!Protocol.IsValid() && !ProtocolName.IsNone())
	{
		const TArray<FName> ProtocolNames = IDMXProtocol::GetProtocolNames();
		if (ProtocolNames.Num() > 0)
		{
			ProtocolName = ProtocolNames[0];
			Protocol = IDMXProtocol::Get(ProtocolName);
		}

		if (!Protocol.IsValid())
		{
			// Mind, while it makes sense for output ports to specify no protocol to internally loopback,  
			// there is no reason to have an input port but no use for it. To the opposite, it may cause
			// undesired behaviour as in 3rd party interference. Hence this is logged as a warning. 
			UE_LOGF(LogDMXProtocol, Warning, "No valid DMX Protocol specified for Input Port %ls. The Port cannot be used.", *PortName);
			return;
		}
	}

	if (Protocol.IsValid())
	{
		// If the extern universe ID is out of the protocol's supported range, mend it.
		ExternUniverseStart = Protocol->MakeValidUniverseID(ExternUniverseStart);

		// Only Local universes > 1 are supported, even if the protocol supports universes < 1.
		LocalUniverseStart = LocalUniverseStart < 1 ? 1 : LocalUniverseStart;

		// Limit the num universes to the max num universes of the protocol
		const int32 MaxNumUniverses = Protocol->GetMaxUniverseID() - Protocol->GetMinUniverseID() + 1;
		NumUniverses = FMath::Min(NumUniverses, MaxNumUniverses);

		// Fix the communication type if it is not supported by the protocol
		TArray<EDMXCommunicationType> CommunicationTypes = Protocol->GetInputPortCommunicationTypes();
		if (!CommunicationTypes.Contains(CommunicationType))
		{
			if (CommunicationTypes.Num() > 0)
			{
				CommunicationType = CommunicationTypes[0];
			}
			else
			{
				// The protocol can specify none to suggest internal only
				CommunicationType = EDMXCommunicationType::InternalOnly;
			}
		}
	}

	if (PortName.IsEmpty())
	{
		UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
		PortName = ProtocolSettings->GetUniqueInputPortName();
	}
}

void FDMXInputPortConfig::Upgrade()
{
#if WITH_EDITOR
	// Initialize the bIsExternUnivereStartEditable property that was added 5.3. Also mends manually edited project ini files.
	bIsExternUnivereStartEditable = ExternUniverseStart != LocalUniverseStart;
#endif
}

FString FDMXInputPortConfig::GetDeviceAddress() const
{
	// Return the Command Line Device Address if it is set
	const FString DMXInputPortCommandLine = FString::Printf(TEXT("dmxinputportip=%s:"), *PortName);
	FString OverrideIP;
	FParse::Value(FCommandLine::Get(), *DMXInputPortCommandLine, OverrideIP);
	if (!OverrideIP.IsEmpty())
	{
		return OverrideIP;
	}

	// Return the Auto Complete Device Address if that's enabled
	if (bAutoCompleteDeviceAddressEnabled)
	{
		FString NetworkInterfaceCardIPAddress;
		if (FDMXProtocolUtils::FindLocalNetworkInterfaceCardIPAddress(AutoCompleteDeviceAddress, NetworkInterfaceCardIPAddress))
		{
			return NetworkInterfaceCardIPAddress;
		}
	}

	// Return the Device Address
	return DeviceAddress;
}
