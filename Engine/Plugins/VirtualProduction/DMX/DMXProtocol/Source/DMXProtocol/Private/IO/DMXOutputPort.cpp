// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/DMXOutputPort.h"

#include "Async/Async.h"
#include "DMXProtocolLog.h"
#include "DMXProtocolSettings.h"
#include "DMXProtocolUtils.h"
#include "HAL/Event.h"
#include "HAL/IConsoleManager.h"
#include "HAL/RunnableThread.h"
#include "Interfaces/IDMXProtocol.h"
#include "Interfaces/IDMXSender.h"
#include "IO/DMXConflictMonitor.h"
#include "IO/DMXOutputPortConfig.h"
#include "IO/DMXPortManager.h"
#include "IO/DMXRawListener.h"
#include "Misc/ExpressionParserTypes.h"
#include "Misc/FrameRate.h"
#include <limits>


#define LOCTEXT_NAMESPACE "DMXOutputPort"



/** Helper to override a member variable of an Output Port */
#define DMX_OVERRIDE_OUTPUTPORT_VAR(MemberName, PortName, Value) \
{ \
	const FDMXOutputPortSharedPtr OverRideOutputPortMacro_OutputPort = UE::DMX::DMXOutputPort::Private::ConsoleCommands::FindOutputPortByName(PortName); \
	if (OverRideOutputPortMacro_OutputPort.IsValid()) \
	{ \
		const FDMXOutputPortConfig OldOutputPortConfig = OverRideOutputPortMacro_OutputPort->MakeOutputPortConfig(); \
		FDMXOutputPortConfigParams OutputPortConfigParams = FDMXOutputPortConfigParams(OldOutputPortConfig); \
		OutputPortConfigParams.MemberName = Value; \
		FDMXOutputPortConfig NewOutputPortConfig(OverRideOutputPortMacro_OutputPort->GetPortGuid(), OutputPortConfigParams); \
		constexpr bool bForceUpdateRegistrationWithProtocol = true; \
		OverRideOutputPortMacro_OutputPort->UpdateFromConfig(NewOutputPortConfig, bForceUpdateRegistrationWithProtocol); \
	} \
}

namespace UE::DMX::DMXOutputPort::Private::ConsoleCommands
{
	/** Logs the configuration of the Output Port */
	void LogOutputPortConfiguration(const FDMXOutputPortSharedPtr OutputPort)
	{
		if (OutputPort.IsValid())
		{
			UE_LOGF(LogDMXProtocol, Log, "================================================");
			UE_LOGF(LogDMXProtocol, Log, "Configuration of Output Port '%ls':", *OutputPort->GetPortName());
			UE_LOGF(LogDMXProtocol, Log, " ");

			UE_LOGF(LogDMXProtocol, Log, "Port Name: %ls", *OutputPort->GetPortName());

			FDMXOutputPortConfig PortConfig = OutputPort->MakeOutputPortConfig();
			UE_LOGF(LogDMXProtocol, Log, "Protocol: %ls", *PortConfig.GetProtocolName().ToString());
			UE_LOGF(LogDMXProtocol, Log, "Communication Type: %i", static_cast<int32>(PortConfig.GetCommunicationType()));
			UE_LOGF(LogDMXProtocol, Log, "TimeToLive: %i", PortConfig.GetTimeToLive());
			UE_LOGF(LogDMXProtocol, Log, "Device Address: %ls", *PortConfig.GetDeviceAddress());

			const TArray<FDMXOutputPortDestinationAddress> DestinationAddresses = PortConfig.GetDestinationAddresses();
			for (int32 IndexDestinationAddress = 0; IndexDestinationAddress < DestinationAddresses.Num(); IndexDestinationAddress++)
			{
				UE_LOGF(LogDMXProtocol, Log, "Destination Address [%i]: %ls", IndexDestinationAddress, *DestinationAddresses[IndexDestinationAddress].DestinationAddressString);
			}

			UE_LOGF(LogDMXProtocol, Log, "Input into Engine: %ls", PortConfig.NeedsLoopbackToEngine() ? TEXT("True") : TEXT("False"));
			UE_LOGF(LogDMXProtocol, Log, "Local Universe Start: %i", PortConfig.GetLocalUniverseStart());
			UE_LOGF(LogDMXProtocol, Log, "Num Universes: %i", PortConfig.GetNumUniverses());
			UE_LOGF(LogDMXProtocol, Log, "Extern Universe Start: %i", PortConfig.GetExternUniverseStart());
			UE_LOGF(LogDMXProtocol, Log, "Priority:	%i", PortConfig.GetPriority());

			UE_LOGF(LogDMXProtocol, Log, "Delay: %f", PortConfig.GetDelay());
			UE_LOGF(LogDMXProtocol, Log, "Delay Frame Rate:	%i/%i", PortConfig.GetDelayFrameRate().Numerator, PortConfig.GetDelayFrameRate().Denominator);

			UE_LOGF(LogDMXProtocol, Log, "Delay in Seconds: %f", OutputPort->GetDelaySeconds());
			UE_LOGF(LogDMXProtocol, Log, "================================================");
		}
	}

	/** Finds an Output port by its name, optionally logging when it can't be found */
	static const FDMXOutputPortSharedPtr FindOutputPortByName(const FString& PortName, bool bPrintToLogIfNotFound = true)
	{
		const FDMXOutputPortSharedRef* OutputPortPtr = FDMXPortManager::Get().GetOutputPorts().FindByPredicate([PortName](const FDMXOutputPortSharedPtr& OutputPort)
			{
				return OutputPort->GetPortName() == PortName;
			});

		if (OutputPortPtr)
		{
			return *OutputPortPtr;
		}
		else if (bPrintToLogIfNotFound)
		{
			if (PortName.IsEmpty())
			{
				UE_LOGF(LogDMXProtocol, Warning, "No Ouptut Port specified for console command. First argument needs to be a valid Port Name.");
			}
			else
			{
				UE_LOGF(LogDMXProtocol, Warning, "Could not find Output Port '%ls'.  Available Output Ports and their names can be found in Project settings -> Plugins -> DMX.", *PortName);
			}
		}

		return nullptr;
	}

	/** Tests the console command, returns true if they yield a valid port. Logs issues otherwise. Handles ? commands. */
	static bool VerifyConsoleCommandArguments(const FString& ConsoleCommand, int32 MinNumExpectedArgs, const TArray<FString>& Args)
	{
		const FString PortName = Args.Num() > 0 ? Args[0] : FString();
		const FDMXOutputPortSharedPtr OutputPort = FindOutputPortByName(PortName);

		const bool bOnlyPortSpecified = Args.Num() == 1;
		const bool bSecondArgIsQuestionmark = (Args.Num() == 2 && Args[1] == TEXT("?"));
		if (OutputPort.IsValid() && (bOnlyPortSpecified || bSecondArgIsQuestionmark))
		{		
			// Log the port if the there is only one arg or the second arg is '?' (e.g.'DMX.SetOutputPortProtocol OutputPortA' or 'DMX.SetOutputPortProtocol OutputPortA ?')
			LogOutputPortConfiguration(OutputPort);

			return false;
		}
		else if (!OutputPort.IsValid())
		{			
			// Log additional help if there were not enough arguments or arguments were invalid
			UE_LOGF(LogDMXProtocol, Warning, "Console Command %ls failed. Output Port with Name '%ls' does not exist.", *ConsoleCommand, *PortName);
			UE_LOGF(LogDMXProtocol, Warning, "Ports and their Names can be found in Project Settings -> Plugins -> DMX.");

			return false;
		}
		else if (MinNumExpectedArgs > Args.Num())
		{
			UE_LOGF(LogDMXProtocol, Warning, "Console Command %ls failed. Invalid arguments specified.", *ConsoleCommand);

			return false;
		}

		return true;
	}

	static FAutoConsoleCommand GDMXLogAllOutputPortConfigurationsCommand(
		TEXT("DMX.LogAllOutputPortConfigurations"),
		TEXT("Logs all Output Port configurations"),
		FConsoleCommandDelegate::CreateStatic(
			[]()
			{
				for (const FDMXOutputPortSharedRef& OutputPort : FDMXPortManager::Get().GetOutputPorts())
				{
					LogOutputPortConfiguration(OutputPort);
				}
			})
	);

	static FAutoConsoleCommand GDMXSetOutputPortProtocolCommand(
		TEXT("DMX.SetOutputPortProtocol"),
		TEXT("DMX.SetOutputPortProtocol [PortName][ProtocolName]. Sets the protocol used by the output port. Example: DMX.SetOutputPortProtocol MyOutputPort Art-Net"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortProtocol"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FName ProtocolNameValue = FName(Args[1]);

				if (IDMXProtocol::GetProtocolNames().Contains(ProtocolNameValue))
				{
					DMX_OVERRIDE_OUTPUTPORT_VAR(ProtocolName, PortName, ProtocolNameValue);
				}
				else
				{
					UE_LOGF(LogDMXProtocol, Warning, "Console command DMX.SetOutputPortProtocol failed. Protocol '%ls' does not exist or is not loaded. Available protocols are: ", *ProtocolNameValue.ToString());
					for (const FName& ProtocolName : IDMXProtocol::GetProtocolNames())
					{
						UE_LOGF(LogDMXProtocol, Warning, "%ls", *ProtocolName.ToString());
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetOutputPortCommunicationTypeCommand(
		TEXT("DMX.SetOutputPortCommunicationType"),
		TEXT("DMX.SetOutputPortCommunicationType [PortName][CommunicationType (0 = Broadcast, 1 = Unicast, 2 = Multicast)]. Sets the communication type of an output port. Example: DMX.SetOutputPortCommunicationType MyOutputPort 2"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortCommunicationType"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FString& CommunicationTypeValueString = Args[1];

				uint8 CommunicationTypeValue;
				if (LexTryParseString<uint8>(CommunicationTypeValue, *CommunicationTypeValueString))
				{
					const EDMXCommunicationType CommunicationTypeEnumValue = static_cast<EDMXCommunicationType>(CommunicationTypeValue);
					if (CommunicationTypeEnumValue == EDMXCommunicationType::Broadcast ||
						CommunicationTypeEnumValue == EDMXCommunicationType::Multicast ||
						CommunicationTypeEnumValue == EDMXCommunicationType::Unicast)
					{
						const FDMXOutputPortSharedPtr OutputPort = FindOutputPortByName(PortName);

						if (OutputPort.IsValid())
						{
							if (const IDMXProtocolPtr Protocol = OutputPort->GetProtocol())
							{
								if (Protocol->GetOutputPortCommunicationTypes().Contains(CommunicationTypeEnumValue))
								{
									DMX_OVERRIDE_OUTPUTPORT_VAR(CommunicationType, PortName, CommunicationTypeEnumValue);
								}
							}
						}
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetTimeToLiveCommand(
		TEXT("DMX.SetOutputPortTimeToLive"),
		TEXT("DMX.SetOutputPortTimeToLive [PortName][TimeToLive (0-255, Default 1)]. Should usually not be changed. Sets how many hops a multicast DMX packet can take before being discarded. Example: DMX.SetOutputPortTimeToLive MyOutputPort 1"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortTimeToLive"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FString& TTLValueString = Args[1];

				uint8 TTLValue;
				if (LexTryParseString<uint8>(TTLValue, *TTLValueString))
				{
					const FDMXOutputPortSharedPtr OutputPort = FindOutputPortByName(PortName);

					if (OutputPort.IsValid())
					{
						if (const IDMXProtocolPtr Protocol = OutputPort->GetProtocol())
						{
							DMX_OVERRIDE_OUTPUTPORT_VAR(TimeToLive, PortName, TTLValue);
						}
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetOutputPortOutputPortDeviceAddressCommand(
		TEXT("DMX.SetOutputPortDeviceAddress"),
		TEXT("DMX.SetOutputPortDeviceAddress [PortName][DeviceAddress]. Sets the Device Address of an output port, usually the network interface card IP address. Example: DMX.SetOutputPortDeviceAddress MyOutputPort 123.45.67.89"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortDeviceAddress"), MinNumExpectedArgs, Args))
				{
					UE_LOGF(LogDMXProtocol, Log, "================================================");
					// For this command, additionally log the available Local Network Interface Card IP addresses
					UE_LOGF(LogDMXProtocol, Log, "Logging available Local Network Interface Card IP Addresses:");
					const TArray<TSharedPtr<FString>> AvailableIPAddresses = FDMXProtocolUtils::GetLocalNetworkInterfaceCardIPs();
					for (int32 IPAddressIndex = 0; IPAddressIndex < AvailableIPAddresses.Num(); IPAddressIndex++)
					{
						UE_LOGF(LogDMXProtocol, Log, "%i: %ls", IPAddressIndex, *(*AvailableIPAddresses[IPAddressIndex]));
					}
					UE_LOGF(LogDMXProtocol, Log, "================================================");
					return;
				}

				const FString& PortName = Args[0];
				const FString& DeviceAddressValue = Args[1];

				DMX_OVERRIDE_OUTPUTPORT_VAR(DeviceAddress, PortName, DeviceAddressValue);
			})
	);

	static FAutoConsoleCommand GDMXSetOutputPortOutputPortDestinationAddressesCommand(
		TEXT("DMX.SetOutputPortDestinationAddresses"),
		TEXT("DMX.SetOutputPortDestinationAddresses [PortName][DestinationAddress1][DestinationAddress2][...][DestinationAddressN]. Sets the Destination Addresses of an output port. Example: DMX.SetOutputPortDeviceAddress MyOutputPort 11.33.55.77 22.44.66.88"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortDestinationAddresses"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				TArray<FDMXOutputPortDestinationAddress>  DestinationAddressesValue;
				for (int32 ArgIndex = 1; ArgIndex < Args.Num(); ArgIndex++)
				{
					const FDMXOutputPortDestinationAddress Address = FDMXOutputPortDestinationAddress(Args[ArgIndex]);
					DestinationAddressesValue.Add(Address);
				}

				DMX_OVERRIDE_OUTPUTPORT_VAR(DestinationAddresses, PortName, DestinationAddressesValue);
			})
	);

	static FAutoConsoleCommand GDMXSetOutputPortLoopbackToEngineCommand(
		TEXT("DMX.SetOutputPortInputIntoEngine"),
		TEXT("DMX.SetOutputPortInputIntoEngine [PortName][Flag]. Sets if the Output Port is input into the engine directly. Example: DMX.SetOutputPortInputIntoEngine MyOutputPort 1"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortInputIntoEngine"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const bool bLoopbackToEngine = Args[1] == TEXT("1") || Args[1].Equals(TEXT("true"), ESearchCase::IgnoreCase);

				DMX_OVERRIDE_OUTPUTPORT_VAR(bLoopbackToEngine, PortName, bLoopbackToEngine);
			})
	);

	static FAutoConsoleCommand GDMXSetOutputPortLocalUniverseStartCommand(
		TEXT("DMX.SetOutputPortLocalUniverseStart"),
		TEXT("DMX.SetOutputPortLocalUniverseStart [PortName][Universe]. Sets the local universe start of the output port. Example: DMX.SetOutputPortLocalUniverseStart MyOutputPort 5"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortLocalUniverseStart"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FDMXOutputPortSharedPtr OutputPort = FindOutputPortByName(PortName);

				if (OutputPort.IsValid())
				{
					if (const IDMXProtocolPtr& Protocol = OutputPort->GetProtocol())
					{
						const FString& LocalUniverseStartValueString = Args[1];
						int32 LocalUniverseStartValue;
						if (LexTryParseString<int32>(LocalUniverseStartValue, *LocalUniverseStartValueString))
						{
							const int32 NumUniverses = OutputPort->GetLocalUniverseEnd() - OutputPort->GetLocalUniverseStart() + 1;

							const int32 ExternUniverseStart = OutputPort->ConvertLocalToExternUniverseID(LocalUniverseStartValue);
							const int32 ExternUniverseEnd = OutputPort->ConvertLocalToExternUniverseID(LocalUniverseStartValue + NumUniverses - 1);

							if (Protocol->IsValidUniverseID(ExternUniverseStart) &&
								Protocol->IsValidUniverseID(ExternUniverseEnd))
							{
								DMX_OVERRIDE_OUTPUTPORT_VAR(LocalUniverseStart, PortName, LocalUniverseStartValue);
							}
							else
							{
								UE_LOGF(LogDMXProtocol, Warning, "Console command DMX.SetOutputPortLocalUniverseStart failed. Local Universe Start '%ls' along with Num Universes '%i' results in a Universe range that is not supported by the Protocol of the Port.", *LocalUniverseStartValueString, NumUniverses);
							}
						}
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetOutputPortNumUniversesCommand(
		TEXT("DMX.SetOutputPortNumUniverses"),
		TEXT("DMX.SetOutputPortNumUniverses [PortName][Universe]. Sets the num universes of the output port. Example: DMX.SetOutputPortNumUniverses MyOutputPort 10"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortNumUniverses"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FDMXOutputPortSharedPtr OutputPort = FindOutputPortByName(PortName);

				if (OutputPort.IsValid())
				{
					if (const IDMXProtocolPtr& Protocol = OutputPort->GetProtocol())
					{
						const FString& NumUniversesValueString = Args[1];
						int32 NumUniversesValue;
						if (LexTryParseString<int32>(NumUniversesValue, *NumUniversesValueString))
						{
							const int32 LocalUniverseStart = OutputPort->GetLocalUniverseStart();

							const int32 ExternUniverseStart = OutputPort->ConvertLocalToExternUniverseID(LocalUniverseStart);
							const int32 ExternUniverseEnd = ExternUniverseStart + NumUniversesValue - 1;

							if (NumUniversesValue > 0 &&
								Protocol->IsValidUniverseID(ExternUniverseStart) &&
								Protocol->IsValidUniverseID(ExternUniverseEnd))
							{
								DMX_OVERRIDE_OUTPUTPORT_VAR(NumUniverses, PortName, NumUniversesValue);
							}
							else
							{
								UE_LOGF(LogDMXProtocol, Warning, "Console command DMX.SetOutputPortNumUniverses failed. Local Universe Start '%i' along with Num Universes '%ls' results in a Universe range that is not supported by the Protocol of the Port.", LocalUniverseStart, *NumUniversesValueString);
							}
						}
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetOutputPortExternUniverseStartCommand(
		TEXT("DMX.SetOutputPortExternUniverseStart"),
		TEXT("DMX.SetOutputPortExternUniverseStart [PortName][Universe]. Sets the extern universe start of the output port. Example: DMX.SetOutputPortExternUniverseStart MyOutputPort 7"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortExternUniverseStart"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FDMXOutputPortSharedPtr OutputPort = FindOutputPortByName(PortName);


				if (OutputPort.IsValid())
				{
					if (const IDMXProtocolPtr& Protocol = OutputPort->GetProtocol())
					{
						const FString& ExternUniverseStartValueString = Args[1];
						int32 ExternUniverseStartValue;
						if (LexTryParseString<int32>(ExternUniverseStartValue, *ExternUniverseStartValueString))
						{
							const int32 NumUniverses = OutputPort->GetLocalUniverseEnd() - OutputPort->GetLocalUniverseStart() + 1;
							const int32 ExternUniverseEnd = ExternUniverseStartValue + NumUniverses - 1;

							if (Protocol->IsValidUniverseID(ExternUniverseStartValue) &&
								Protocol->IsValidUniverseID(ExternUniverseEnd))
							{
								DMX_OVERRIDE_OUTPUTPORT_VAR(ExternUniverseStart, PortName, ExternUniverseStartValue);
							}
							else
							{
								UE_LOGF(LogDMXProtocol, Warning, "Console command DMX.SetOutputPortExternUniverseStart failed. Extern Universe Start '%ls' along with Num Universes '%i' results in a Universe range that is not supported by the Protocol of the Port.", *ExternUniverseStartValueString, NumUniverses);
							}
						}
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXSetOutputPortPriorityCommand(
		TEXT("DMX.SetOutputPortPriority"),
		TEXT("DMX.SetOutputPortPriority [PortName][Priority]. Sets the priority of the output port. Example: DMX.SetOutputPortPriority MyOutputPort 100"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortPriority"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FString& PriorityValueString = Args[1];
				int32 PriorityValue;
				if (LexTryParseString<int32>(PriorityValue, *PriorityValueString))
				{
					DMX_OVERRIDE_OUTPUTPORT_VAR(Priority, PortName, PriorityValue);
				}
			})
	);

	static FAutoConsoleCommand GDMXSetOutputPortDelayCommand(
		TEXT("DMX.SetOutputPortDelay"),
		TEXT("DMX.SetOutputPortDelay [PortName][Delay][(optional)FrameRate]. Sets the delay of the output port, optionally with a frame rate (e.g. '30fps', '0.001s' or '12000/1001'). Example: DMX.SetOutputPortDelay MyOutputPort 10.5 30fps"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 2;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.SetOutputPortDelay"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FString& DelayValueString = Args[1];
				double DelayValue;
				if (LexTryParseString<double>(DelayValue, *DelayValueString))
				{
					FFrameRate NewFrameRate = FFrameRate(1, 1);

					// Parse optional frame rate argument
					if (Args.Num() == 3)
					{
						const FString& DelayFrameRateString = Args[2];

						TValueOrError<FFrameRate, FExpressionError> ParseResult = ParseFrameRate(*DelayFrameRateString);
						if (ParseResult.IsValid())
						{
							NewFrameRate = ParseResult.GetValue();
						}
						else
						{
							UE_LOGF(LogDMXProtocol, Warning, "Could not parse frame rate from console command DMX.SetOutputPortDelay. Note, not all decimal numbers are supported. E.g. 0.345s needs to be written as 345/1000.")
						}
					}

					if (DelayValue >= 0.0)
					{
						const FDMXOutputPortSharedPtr OutputPort = FindOutputPortByName(PortName);

						if (OutputPort.IsValid())
						{
							DMX_OVERRIDE_OUTPUTPORT_VAR(DelayFrameRate, PortName, NewFrameRate);
							DMX_OVERRIDE_OUTPUTPORT_VAR(Delay, PortName, DelayValue);
						}
					}
				}
			})
	);

	static FAutoConsoleCommand GDMXResetOutputPortToProjectSettings(
		TEXT("DMX.ResetOutputPortToProjectSettings"),
		TEXT("DMX.ResetOutputPortToProjectSettings [PortName]. Resets the output port to how it is defined in project settings. Example: DMX.ResetOutputPortToProjectSettings MyOutputPort"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				constexpr int32 MinNumExpectedArgs = 1;
				if (!VerifyConsoleCommandArguments(TEXT("DMX.ResetOutputPortToProjectSettings"), MinNumExpectedArgs, Args))
				{
					return;
				}

				const FString& PortName = Args[0];
				const FDMXOutputPortSharedPtr OutputPort = FindOutputPortByName(PortName);

				if(OutputPort.IsValid())
				{
					const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
					if (ProtocolSettings)
					{
						const FDMXOutputPortConfig* PortConfigPtr = ProtocolSettings->OutputPortConfigs.FindByPredicate([OutputPort](const FDMXOutputPortConfig& OutputPortConfig)
							{
								return OutputPortConfig.GetPortGuid() == OutputPort->GetPortGuid();
							});

						if (PortConfigPtr)
						{
							FDMXOutputPortConfig PortConfig = *PortConfigPtr;
							OutputPort->UpdateFromConfig(PortConfig);
						}
					}
				}
			})
	);
}
#undef DMX_OVERRIDE_OUTPUTPORT_VAR

FDMXOutputPortSharedRef FDMXOutputPort::CreateFromConfig(FDMXOutputPortConfig& OutputPortConfig)
{
	checkf(IsInGameThread(), TEXT("Only game-thread can create ports."));

	// Port Configs are expected to have a valid guid always
	check(OutputPortConfig.GetPortGuid().IsValid());

	FDMXOutputPortSharedRef NewOutputPort = MakeShared<FDMXOutputPort, ESPMode::ThreadSafe>();

	NewOutputPort->SendDMXEvent = FPlatformProcess::GetSynchEventFromPool();
	check(NewOutputPort->SendDMXEvent != nullptr);

	NewOutputPort->PortGuid = OutputPortConfig.GetPortGuid();

	UDMXProtocolSettings* Settings = GetMutableDefault<UDMXProtocolSettings>();
	check(Settings);

	NewOutputPort->SendRate = Settings->SendingRefreshRate;
	NewOutputPort->CommunicationDeterminator.SetSendEnabled(Settings->IsSendDMXEnabled());
	NewOutputPort->CommunicationDeterminator.SetReceiveEnabled(Settings->IsReceiveDMXEnabled());

	Settings->GetOnSendingRefresRateChanged().AddThreadSafeSP(NewOutputPort, &FDMXOutputPort::OnSendingRefreshRateChanged);
	Settings->GetOnSetSendDMXEnabled().AddThreadSafeSP(NewOutputPort, &FDMXOutputPort::OnSetSendDMXEnabled);
	Settings->GetOnSetReceiveDMXEnabled().AddThreadSafeSP(NewOutputPort, &FDMXOutputPort::OnSetReceiveDMXEnabled);
	NewOutputPort->UpdateFromConfig(OutputPortConfig);

	const FString SenderThreadName = FString(TEXT("DMXOutputPort_")) + OutputPortConfig.GetPortName();
	NewOutputPort->Thread = FRunnableThread::Create(&NewOutputPort.Get(), *SenderThreadName, 0U, TPri_TimeCritical, FPlatformAffinity::GetPoolThreadMask());

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Created output port %ls", *NewOutputPort->PortName);

	return NewOutputPort;
}

FDMXOutputPort::~FDMXOutputPort()
{	
	// Port needs be unregistered before destruction
	check(DMXSenderArray.Num() == 0);

	if (Thread)
	{
		Thread->Kill(true);
		delete Thread;
	}

	FPlatformProcess::ReturnSynchEventToPool(SendDMXEvent);
	SendDMXEvent = nullptr;

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Destroyed output port %ls", *PortName);
}

FDMXOutputPortConfig FDMXOutputPort::MakeOutputPortConfig() const
{
	checkf(IsInGameThread(), TEXT("Only game-thread can read port configs."));

	const FScopeLock Lock(&SendDMXMutex);

	FDMXOutputPortConfigParams Params;
	Params.PortName = PortName;
	Params.ProtocolName = Protocol.IsValid() ? Protocol->GetProtocolName() : NAME_None;
	Params.CommunicationType = CommunicationType;
	Params.bAutoCompleteDeviceAddressEnabled = bAutoCompleteDeviceAddressEnabled;
	Params.AutoCompleteDeviceAddress = AutoCompleteDeviceAddress;
	Params.DeviceAddress = DeviceAddress;
	Params.DestinationAddresses = DestinationAddresses;
	Params.bLoopbackToEngine = CommunicationDeterminator.IsLoopbackToEngineEnabled();
	Params.LocalUniverseStart = LocalUniverseStart;
	Params.NumUniverses = NumUniverses;
	Params.ExternUniverseStart = ExternUniverseStart;
	Params.Priority = Priority;
	Params.Delay = DelaySeconds * DelayFrameRate.AsDecimal();
	Params.DelayFrameRate = DelayFrameRate;

	return FDMXOutputPortConfig(PortGuid, Params);
}

void FDMXOutputPort::UpdateFromConfig(FDMXOutputPortConfig& InOutOutputPortConfig, bool bForceUpdateRegistrationWithProtocol)
{	
	checkf(IsInGameThread(), TEXT("Only game-thread can update a port from a config."));

	const FScopeLock Lock(&SendDMXMutex);

	// Need a valid config for the port
	InOutOutputPortConfig.MakeValid();

	// Avoid further changes to the config
	const FDMXOutputPortConfig& OutputPortConfig = InOutOutputPortConfig;

	// Can only use configs that correspond to project settings
	const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
	const bool bConfigIsInProjectSettings = ProtocolSettings->OutputPortConfigs.ContainsByPredicate([&OutputPortConfig](const FDMXOutputPortConfig& Other) {
		return OutputPortConfig.GetPortGuid() == Other.GetPortGuid();
	});
	ensureAlwaysMsgf(bConfigIsInProjectSettings, TEXT("Can only use configs with a guid that corresponds to a config in project settings"));

	// Find if the port needs update its registration with the protocol
	const bool bNeedsUpdateRegistration = [this, &OutputPortConfig, bForceUpdateRegistrationWithProtocol]()
	{
		if (bForceUpdateRegistrationWithProtocol)
		{
			return true;
		}

		if (IsRegistered() != CommunicationDeterminator.IsSendDMXEnabled())
		{
			return true;
		}

		FName ProtocolName = Protocol.IsValid() ? Protocol->GetProtocolName() : NAME_None;

		if (ProtocolName == OutputPortConfig.GetProtocolName() &&
			bAutoCompleteDeviceAddressEnabled == OutputPortConfig.IsAutoCompleteDeviceAddressEnabled() &&
			AutoCompleteDeviceAddress == OutputPortConfig.GetAutoCompleteDeviceAddress() &&
			DeviceAddress == OutputPortConfig.GetDeviceAddress() &&
			DestinationAddresses == OutputPortConfig.GetDestinationAddresses() &&
			CommunicationType == OutputPortConfig.GetCommunicationType() &&
			Priority == OutputPortConfig.GetPriority() &&
			DelaySeconds == OutputPortConfig.GetDelay() * OutputPortConfig.GetDelayFrameRate().AsDecimal() &&
			TimeToLive == OutputPortConfig.GetTimeToLive())
		{
			return false;
		}	

		return true;
	}();

	// Unregister the port if required before the new protocol is set
	if (bNeedsUpdateRegistration)
	{
		if (IsRegistered())
		{
			Unregister();
		}
	}

	Protocol = IDMXProtocol::Get(OutputPortConfig.GetProtocolName());

	// Copy properties from the config
	const FGuid& ConfigPortGuid = OutputPortConfig.GetPortGuid();
	check(PortGuid.IsValid());
	PortGuid = ConfigPortGuid;

	CommunicationType = OutputPortConfig.GetCommunicationType();
	TimeToLive = OutputPortConfig.GetTimeToLive();
	bAutoCompleteDeviceAddressEnabled = OutputPortConfig.IsAutoCompleteDeviceAddressEnabled();
	AutoCompleteDeviceAddress = OutputPortConfig.GetAutoCompleteDeviceAddress();
	DeviceAddress = OutputPortConfig.GetDeviceAddress();
	DestinationAddresses = OutputPortConfig.GetDestinationAddresses();
	LocalUniverseStart = OutputPortConfig.GetLocalUniverseStart();
	NumUniverses = OutputPortConfig.GetNumUniverses();
	ExternUniverseStart = OutputPortConfig.GetExternUniverseStart();
	PortName = OutputPortConfig.GetPortName();
	Priority = OutputPortConfig.GetPriority();
	DelaySeconds = OutputPortConfig.GetDelay() / OutputPortConfig.GetDelayFrameRate().AsDecimal();
	DelayFrameRate = OutputPortConfig.GetDelayFrameRate();

	CommunicationDeterminator.SetLoopbackToEngine(OutputPortConfig.NeedsLoopbackToEngine());

	// Only update on the next tick to give the OS time to fully release previous sockets
	if (bPendingRegistration)
	{
		return;
	}
	bPendingRegistration = true;

	AsyncTask(ENamedThreads::GameThread, [bNeedsUpdateRegistration, WeakThis = AsWeak(), this]()
		{
			if (!WeakThis.IsValid())
			{
				return;
			}

			// Re-register the port if required
			if (bNeedsUpdateRegistration)
			{
				Register();
			}

			bPendingRegistration = false;

			OnPortUpdated.Broadcast();
		});
}

const FGuid& FDMXOutputPort::GetPortGuid() const
{
	check(PortGuid.IsValid());
	return PortGuid;
}

void FDMXOutputPort::ClearBuffers()
{
	// Note, it is important that this is called from the game thread!
	// Otherwise the Send methods would need a lock on enqueuing, deterring the lock-free implementation.
	checkf(IsInGameThread(), TEXT("Only the game-thread can ever clear buffers."));

	const FScopeLock Lock(&SendDMXMutex);

	ExternUniverseToLatestSignalMap_GameThread.Reset();
	ExternUniverseToLatestSignalMap_PortThread.Reset();

	SignalFragments.Empty();

	for (const TSharedRef<FDMXRawListener>& RawListener : RawListeners)
	{
		RawListener->ClearBuffer();
	}
}

TArray<FString> FDMXOutputPort::GetDestinationAddresses() const
{
	const FScopeLock Lock(&SendDMXMutex);

	TArray<FString> Result;
	Result.Reserve(DestinationAddresses.Num());
	for (const FDMXOutputPortDestinationAddress& DestinationAddress : DestinationAddresses)
	{
		Result.Add(DestinationAddress.DestinationAddressString);
	}

	return Result;
}

bool FDMXOutputPort::GameThreadGetDMXSignal(int32 LocalUniverseID, FDMXSignalSharedPtr& OutDMXSignal, bool bEvenIfNotLoopbackToEngine)
{
	check(IsInGameThread());

	const bool bNeedsLoopbackToEngine = CommunicationDeterminator.NeedsLoopbackToEngine();
	if (bNeedsLoopbackToEngine || bEvenIfNotLoopbackToEngine)
	{
		int32 ExternUniverseID = ConvertLocalToExternUniverseID(LocalUniverseID);

		const FDMXSignalSharedPtr* SignalPtr = ExternUniverseToLatestSignalMap_GameThread.Find(ExternUniverseID);
		if (SignalPtr)
		{
			OutDMXSignal = *SignalPtr;
			return true;
		}
	}

	return false;
}

bool FDMXOutputPort::GameThreadGetDMXSignalFromRemoteUniverse(FDMXSignalSharedPtr& OutDMXSignal, int32 RemoteUniverseID, bool bEvenIfNotLoopbackToEngine)
{
	// DEPRECATED 4.27
	check(IsInGameThread());

	const bool bNeedsLoopbackToEngine = CommunicationDeterminator.NeedsLoopbackToEngine();
	if (bNeedsLoopbackToEngine || bEvenIfNotLoopbackToEngine)
	{
		const FDMXSignalSharedPtr* SignalPtr = ExternUniverseToLatestSignalMap_GameThread.Find(RemoteUniverseID);
		if (SignalPtr)
		{
			OutDMXSignal = *SignalPtr;
			return true;
		}
	}

	return false;
}

bool FDMXOutputPort::IsRegistered() const
{	
	return bRegistered;
}

void FDMXOutputPort::AddRawListener(TSharedRef<FDMXRawListener> InRawListener)
{
	const FScopeLock Lock(&SendDMXMutex);

	check(!RawListeners.Contains(InRawListener));

	RawListeners.Add(InRawListener);
}

void FDMXOutputPort::RemoveRawListener(TSharedRef<FDMXRawListener> InRawListenerToRemove)
{
	const FScopeLock Lock(&SendDMXMutex);

	RawListeners.Remove(InRawListenerToRemove);
}

void FDMXOutputPort::SendDMX(int32 LocalUniverseID, const TMap<int32, uint8>& ChannelToValueMap)
{
	using namespace UE::DMX;
	checkf(IsInGameThread(), TEXT("Only the game-thread can Send from a DMX Output Port."));

	if (IsLocalUniverseInPortRange(LocalUniverseID))
	{
		// We don't mind to lock the communication determinator, it is ok if it is not sync while changing
		const bool bNeedsSendDMX = CommunicationDeterminator.NeedsSendDMX();
		const bool bNeedsLoopbackToEngine = CommunicationDeterminator.NeedsLoopbackToEngine();

		// Update the buffer for loopback if dmx needs be sent and/or looped back
		if (bNeedsSendDMX || bNeedsLoopbackToEngine)
		{
			const int32 ExternUniverseID = ConvertLocalToExternUniverseID(LocalUniverseID);

			const double SendTime = FPlatformTime::Seconds() + DelaySeconds;

			// Enqueue for this port's thread. Thread-safe without locking.
			const TSharedPtr<FDMXSignalFragment> Fragment = MakeShared<FDMXSignalFragment>(ExternUniverseID, ChannelToValueMap, SendTime);	
			SignalFragments.Enqueue(Fragment);

			// Write the fragment to the game thread's buffer
			const FDMXSignalSharedPtr& Signal = ExternUniverseToLatestSignalMap_GameThread.FindOrAdd(ExternUniverseID, MakeShared<FDMXSignal, ESPMode::ThreadSafe>());

#if WITH_EDITOR
			// Monitor conflicts
			if (FDMXConflictMonitor* ConflictMonitor = FDMXConflictMonitor::Get())
			{
				ConflictMonitor->MonitorOutboundDMX(SharedThis(this), LocalUniverseID, ChannelToValueMap);
			}
#endif

			for (const TTuple<int32, uint8>& ChannelValueKvp : ChannelToValueMap)
			{
				int32 ChannelIndex = ChannelValueKvp.Key - 1;

				// Filter invalid indicies so we can send bp calls here without testing them first.
				if (Signal->ChannelData.IsValidIndex(ChannelIndex))
				{
					Signal->Timestamp = SendTime;
					Signal->ChannelData[ChannelIndex] = ChannelValueKvp.Value;
				}
			}
		}
	}
}

void FDMXOutputPort::SendDMXToRemoteUniverse(const TMap<int32, uint8>& ChannelToValueMap, int32 RemoteUniverse)
{
	// DEPRECATED 4.27
	check(IsInGameThread());

	if (IsExternUniverseInPortRange(RemoteUniverse))
	{
		const bool bNeedsSendDMX = CommunicationDeterminator.NeedsSendDMX();
		const bool bNeedsLoopbackToEngine = CommunicationDeterminator.NeedsLoopbackToEngine();

		// Update the buffer for loopback if dmx needs be sent and/or looped back
		if (bNeedsSendDMX || bNeedsLoopbackToEngine)
		{
			const double SendTime = FPlatformTime::Seconds() + DelaySeconds;

			// Enqueue for this port's thread
			const TSharedPtr<FDMXSignalFragment> Fragment = MakeShared<FDMXSignalFragment>(RemoteUniverse, ChannelToValueMap, SendTime);
			SignalFragments.Enqueue(Fragment);

			if (bNeedsLoopbackToEngine)
			{
				// Write the fragment to the game thread's buffer
				const FDMXSignalSharedPtr& Signal = ExternUniverseToLatestSignalMap_GameThread.FindOrAdd(RemoteUniverse, MakeShared<FDMXSignal, ESPMode::ThreadSafe>());

				for (const TTuple<int32, uint8>& ChannelValueKvp : ChannelToValueMap)
				{
					int32 ChannelIndex = ChannelValueKvp.Key - 1;

					// Filter invalid indicies so we can send bp calls here without testing them first.
					if (Signal->ChannelData.IsValidIndex(ChannelIndex))
					{
						Signal->Timestamp = SendTime;
						Signal->ChannelData[ChannelIndex] = ChannelValueKvp.Value;
					}
				}
			}
		}
	}
}

bool FDMXOutputPort::Register()
{
	checkf(IsInGameThread(), TEXT("Only Game Thread can register output ports."));

	const FScopeLock Lock(&SendDMXMutex);

	if (Protocol.IsValid() && IsValidPortSlow() && CommunicationDeterminator.IsSendDMXEnabled() && !FDMXPortManager::Get().AreProtocolsSuspended())
	{
		DMXSenderArray = Protocol->RegisterOutputPort(SharedThis(this));

		if (DMXSenderArray.Num() > 0)
		{
			CommunicationDeterminator.SetHasValidSender(true);
			bRegistered = true;

			return true;
		}
	}

	CommunicationDeterminator.SetHasValidSender(false);
	bRegistered = false;

	return false;
}

void FDMXOutputPort::Unregister()
{
	checkf(IsInGameThread(), TEXT("Only Game Thread can unregister output ports."));

	const FScopeLock Lock(&SendDMXMutex);

	if (IsRegistered())
	{
		if (Protocol.IsValid())
		{
			Protocol->UnregisterOutputPort(SharedThis(this));
		}

		DMXSenderArray.Reset();
	}

	CommunicationDeterminator.SetHasValidSender(false);
	bRegistered = false;
}

void FDMXOutputPort::OnSendingRefreshRateChanged()
{
	const UDMXProtocolSettings* ProtocolSettings = GetDefault< UDMXProtocolSettings>();
	SendRate = ProtocolSettings->SendingRefreshRate;
}

void FDMXOutputPort::OnSetSendDMXEnabled(bool bEnabled)
{
	checkf(IsInGameThread(), TEXT("Only the game thread can enable and disable send DMX."));

	CommunicationDeterminator.SetSendEnabled(bEnabled);

	FDMXOutputPortConfig Config = MakeOutputPortConfig();
	UpdateFromConfig(Config);
}

void FDMXOutputPort::OnSetReceiveDMXEnabled(bool bEnabled)
{
	checkf(IsInGameThread(), TEXT("Only the game thread can enable and disable send DMX."));

	CommunicationDeterminator.SetReceiveEnabled(bEnabled);

	FDMXOutputPortConfig Config = MakeOutputPortConfig();
	UpdateFromConfig(Config);
}

bool FDMXOutputPort::Init()
{
	return true;
}

uint32 FDMXOutputPort::Run()
{
	while (!bStopping)
	{
		const double StartTime = FPlatformTime::Seconds();
		if (IsEngineExitRequested())
		{
			// Stop sending DMX when the engine is shutting down
			break;
		}

		ProcessSendDMX();

		const double Interval = 1.0 / SendRate;
		const double EndTime = FPlatformTime::Seconds();
		const double Elapsed = EndTime - StartTime;

		if (Interval > Elapsed)
		{
			const FTimespan WaitTimespan = FTimespan::FromSeconds(Interval - Elapsed);
			SendDMXEvent->Wait(WaitTimespan);
		}
	}

	return 0;
}

void FDMXOutputPort::Stop()
{
	bStopping = true;

	// Effectively trigger the stop
	if (SendDMXEvent)
	{
		SendDMXEvent->Trigger();
	}
}

void FDMXOutputPort::Exit()
{
}

void FDMXOutputPort::Tick()
{
	ProcessSendDMX();
}

FSingleThreadRunnable* FDMXOutputPort::GetSingleThreadInterface()
{
	return this;
}

void FDMXOutputPort::ProcessSendDMX()
{
	// Delay signals
	const double Now = FPlatformTime::Seconds();

	// Locking Overhead here is minimal as this is only locked while changing port properties.
	const FScopeLock Lock(&SendDMXMutex);

	// Write dmx fragments
	for (;;)
	{
		TSharedPtr<FDMXSignalFragment, ESPMode::ThreadSafe> OldestFragment;
		if (SignalFragments.Peek(OldestFragment))
		{
			if (OldestFragment->SendTime <= Now)
			{
				const FDMXSignalSharedPtr& Signal = ExternUniverseToLatestSignalMap_PortThread.FindOrAdd(OldestFragment->ExternUniverseID, MakeShared<FDMXSignal, ESPMode::ThreadSafe>());

				// Write the fragment & meta data 
				for (const TTuple<int32, uint8>& ChannelValueKvp : OldestFragment->ChannelToValueMap)
				{
					int32 ChannelIndex = ChannelValueKvp.Key - 1;
					// Filter invalid indicies so we can send bp calls here without testing them first.
					if (Signal->ChannelData.IsValidIndex(ChannelIndex))
					{
						Signal->ChannelData[ChannelIndex] = ChannelValueKvp.Value;
					}
				}

				Signal->ExternUniverseID = OldestFragment->ExternUniverseID;
				Signal->Timestamp = Now;
				Signal->Priority = Priority;

				// Drop the written fragment
				SignalFragments.Pop();

				continue;
			}

			break;
		}

		break;
	}

	// Send new and alive DMX Signals
	const bool bNeedsSendDMX = CommunicationDeterminator.NeedsSendDMX();

	for (const TTuple<int32, FDMXSignalSharedPtr>& UniverseToSignalPair : ExternUniverseToLatestSignalMap_PortThread)
	{
		if (UniverseToSignalPair.Value->Timestamp <= Now)
		{
			// Keeping the signal alive here:
			// Increment the timestamp by one second so the signal will be sent anew in one second.
			UniverseToSignalPair.Value->Timestamp = Now + 1.0;

			// Send via the protocol's sender
			if (bNeedsSendDMX)
			{
				for (const TSharedPtr<IDMXSender>& DMXSender : DMXSenderArray)
				{
					DMXSender->SendDMXSignal(UniverseToSignalPair.Value.ToSharedRef());
				}
			}

			// Loopback to Listeners
			for (const TSharedRef<FDMXRawListener>& RawListener : RawListeners)
			{
				RawListener->EnqueueSignal(this, UniverseToSignalPair.Value.ToSharedRef());
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
