// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/DMXOutputPortConfig.h"

#include "DMXProtocolSettings.h"
#include "DMXProtocolUtils.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXPortManager.h"

#include "Misc/Guid.h"


FDMXOutputPortConfigParams::FDMXOutputPortConfigParams(const FDMXOutputPortConfig& OutputPortConfig)
	: PortName(OutputPortConfig.GetPortName())
	, ProtocolName(OutputPortConfig.GetProtocolName())
	, CommunicationType(OutputPortConfig.GetCommunicationType())
	, TimeToLive(OutputPortConfig.GetTimeToLive())
	, bAutoCompleteDeviceAddressEnabled(OutputPortConfig.IsAutoCompleteDeviceAddressEnabled())
	, AutoCompleteDeviceAddress(OutputPortConfig.GetAutoCompleteDeviceAddress())
	, DeviceAddress(OutputPortConfig.GetDeviceAddress())
	, DestinationAddresses(OutputPortConfig.GetDestinationAddresses())
	, bLoopbackToEngine(OutputPortConfig.NeedsLoopbackToEngine())
	, LocalUniverseStart(OutputPortConfig.GetLocalUniverseStart())
	, NumUniverses(OutputPortConfig.GetNumUniverses())
	, ExternUniverseStart(OutputPortConfig.GetExternUniverseStart())
	, Priority(OutputPortConfig.GetPriority())
	, Delay(OutputPortConfig.GetDelay())
	, DelayFrameRate(OutputPortConfig.GetDelayFrameRate())
{}


FDMXOutputPortConfig::FDMXOutputPortConfig()
	: DelayFrameRate(FFrameRate(1.0, 1.0)) // Default delay frame rate to 1.0 (default to Seconds)
	, PortGuid(FGuid::NewGuid())
{}

FDMXOutputPortConfig::FDMXOutputPortConfig(const FGuid& InPortGuid)
	: DelayFrameRate(FFrameRate(1.0, 1.0)) // Default delay frame rate to 1.0 (default to Seconds)
	, PortGuid(InPortGuid)
{
	// Cannot create port configs before the protocol module is up (it is required to sanetize protocol names).
	check(FModuleManager::Get().IsModuleLoaded("DMXProtocol"));
	check(PortGuid.IsValid());

	MakeValid();
}

FDMXOutputPortConfig::FDMXOutputPortConfig(const FGuid& InPortGuid, const FDMXOutputPortConfigParams& InitializationData)
	: PortName(InitializationData.PortName)
	, ProtocolName(InitializationData.ProtocolName)
	, CommunicationType(InitializationData.CommunicationType)
	, TimeToLive(InitializationData.TimeToLive)
	, bAutoCompleteDeviceAddressEnabled(InitializationData.bAutoCompleteDeviceAddressEnabled)
	, AutoCompleteDeviceAddress(InitializationData.AutoCompleteDeviceAddress)
	, DeviceAddress(InitializationData.DeviceAddress)
	, DestinationAddresses(InitializationData.DestinationAddresses)
	, bLoopbackToEngine(InitializationData.bLoopbackToEngine)
	, LocalUniverseStart(InitializationData.LocalUniverseStart)
	, NumUniverses(InitializationData.NumUniverses)
	, ExternUniverseStart(InitializationData.ExternUniverseStart)
	, Priority(InitializationData.Priority)
	, Delay(InitializationData.Delay)
	, DelayFrameRate(InitializationData.DelayFrameRate)
	, PortGuid(InPortGuid)
{
	// Cannot create port configs before the protocol module is up (it is required to sanetize protocol names).
	check(FModuleManager::Get().IsModuleLoaded("DMXProtocol"));

	check(PortGuid.IsValid());
	check(!ProtocolName.IsNone())

	MakeValid();
}

void FDMXOutputPortConfig::MakeValid()
{
	if (!ensureAlwaysMsgf(PortGuid.IsValid(), TEXT("Invalid GUID for Input Port %s. Generating a new one. Blueprint nodes referencing the port will no longer be functional."), *PortName))
	{
		PortGuid = FGuid::NewGuid();
	}

	// Try to restore the protocol if it is not valid.
	IDMXProtocolPtr Protocol = IDMXProtocol::Get(ProtocolName);	
	if (!Protocol.IsValid())
	{
		const TArray<FName> ProtocolNames = IDMXProtocol::GetProtocolNames();
		if (ProtocolNames.Num() > 0)
		{
			ProtocolName = ProtocolNames[0];
			Protocol = IDMXProtocol::Get(ProtocolName);
		}

		if (!Protocol.IsValid())
		{
			// Accept NAME_None was specified as a protocol, but log that it will only be useful for internal loopback.
			// This is a temporary solution for projects that want to use DMX, but not send or receive DMX over the network.
			UE_LOGF(LogDMXProtocol, Log, "No protocol specified for Output Port %ls. The Port can be used for internal loopback only.", *PortName);
			return;
		}
	}

	if (Protocol.IsValid())
	{
		// If the extern universe ID is out of the protocol's supported range, mend it.
		ExternUniverseStart = Protocol->MakeValidUniverseID(ExternUniverseStart);

		// Only Local universes > 1 are supported, even if the protocol supports universes < 1.
		LocalUniverseStart = LocalUniverseStart < 1 ? 1 : LocalUniverseStart;

		// Limit the num universes relatively to the max extern universe of the protocol and int32 max
		const int64 MaxNumUniverses = Protocol->GetMaxUniverseID() - Protocol->GetMinUniverseID() + 1;
		const int64 DesiredNumUniverses = NumUniverses > MaxNumUniverses ? MaxNumUniverses : NumUniverses;
		const int64 DesiredLocalUniverseEnd = static_cast<int64>(LocalUniverseStart) + DesiredNumUniverses - 1;

		if (DesiredLocalUniverseEnd > TNumericLimits<int32>::Max())
		{
			NumUniverses = TNumericLimits<int32>::Max() - DesiredLocalUniverseEnd;
		}

		// Fix the communication type if it is not supported by the protocol
		TArray<EDMXCommunicationType> CommunicationTypes = Protocol->GetOutputPortCommunicationTypes();
		if (!CommunicationTypes.Contains(CommunicationType))
		{
			if (CommunicationTypes.Num() > 0)
			{
				CommunicationType = CommunicationTypes[0];
			}
			else
			{
				// The protocol can specify none to suggest internal only
				CommunicationType = EDMXCommunicationType::InternalOnly;
			}
		}
	}

	// Allow for postitive delay values only
	if (Delay < 0.0)
	{
		Delay = 0.0;
	}

	if (PortName.IsEmpty())
	{
		UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
		PortName = ProtocolSettings->GetUniqueOutputPortName();
	}
}

void FDMXOutputPortConfig::Upgrade()
{
#if WITH_EDITOR
	// Initialize the bIsExternUnivereStartEditable property that was added 5.3. Also mends manually edited project ini files.
	bIsExternUnivereStartEditable = ExternUniverseStart != LocalUniverseStart;
#endif
}

FString FDMXOutputPortConfig::GetDeviceAddress() const
{
	// Return the Command Line Device Address if it is set
	const FString DMXOutputPortCommandLine = FString::Printf(TEXT("dmxoutputportip=%s:"), *PortName);
	FString OverrideIP;
	FParse::Value(FCommandLine::Get(), *DMXOutputPortCommandLine, OverrideIP);
	if (!OverrideIP.IsEmpty())
	{
		return OverrideIP;
	}

	// Return the Auto Complete Device Address if that's enabled
	if (bAutoCompleteDeviceAddressEnabled)
	{
		FString NetworkInterfaceCardIPAddress;
		if (FDMXProtocolUtils::FindLocalNetworkInterfaceCardIPAddress(AutoCompleteDeviceAddress, NetworkInterfaceCardIPAddress))
		{
			return NetworkInterfaceCardIPAddress;
		}
	}

	// Return the Device Address
	return DeviceAddress;
}
