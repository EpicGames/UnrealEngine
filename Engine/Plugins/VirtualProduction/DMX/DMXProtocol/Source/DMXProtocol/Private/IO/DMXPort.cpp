// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/DMXPort.h"

#include "DMXProtocolConstants.h"
#include "DMXProtocolLog.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXPortManager.h"

#include "SocketSubsystem.h"
#include "IPAddress.h"
#include "Misc/Guid.h"


bool FDMXPort::IsLocalUniverseInPortRange(int32 LocalUniverse) const
{
	if (LocalUniverse >= LocalUniverseStart &&
		LocalUniverse <= LocalUniverseStart + NumUniverses - 1)
	{
		return true;
	}

	return false;
}

bool FDMXPort::IsExternUniverseInPortRange(int32 ExternUniverse) const
{
	if (ExternUniverse >= ExternUniverseStart &&
		ExternUniverse <= ExternUniverseStart + NumUniverses - 1)
	{
		return true;
	}

	return false;
}

int32 FDMXPort::GetExternUniverseOffset() const
{
	return ExternUniverseStart - LocalUniverseStart;
}

int32 FDMXPort::ConvertExternToLocalUniverseID(int32 ExternUniverseID) const
{
	return ExternUniverseID - GetExternUniverseOffset();
}

int32 FDMXPort::ConvertLocalToExternUniverseID(int32 LocalUniverseID) const
{
	return LocalUniverseID + GetExternUniverseOffset();
}

bool FDMXPort::IsValidPortSlow() const
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	TSharedPtr<FInternetAddr> IPInternetAddr = SocketSubsystem->CreateInternetAddr();

	if (ExternUniverseStart < Protocol->GetMinUniverseID())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create DMX Port: Resulting First Universe %i is smaller than Protocol Min %i.", ExternUniverseStart, Protocol->GetMinUniverseID());
		return false;
	}

	const int32 ExternUniverseEnd = ExternUniverseStart + NumUniverses - 1;
	if (ExternUniverseEnd > Protocol->GetMaxUniverseID())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create DMX Port: Resulting Last Universe %i is bigger than Protocol Max %i.", ExternUniverseEnd, Protocol->GetMinUniverseID());
		return false;
	}

	return true;
}
