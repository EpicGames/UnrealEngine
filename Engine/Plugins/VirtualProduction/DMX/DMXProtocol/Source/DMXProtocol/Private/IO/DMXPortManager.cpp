// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/DMXPortManager.h"

#include "DMXProtocolSettings.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXInputPortConfig.h"
#include "IO/DMXOutputPortConfig.h"

#include "Templates/UniquePtr.h"
#include "UObject/UnrealType.h"


TUniquePtr<FDMXPortManager> FDMXPortManager::CurrentManager;

FDMXPortManager::~FDMXPortManager()
{
	checkf(!CurrentManager.IsValid(), TEXT("ShutdownManager was not called"));

	// Other objects may still hold shared pointers to ports up until engine shuts down.
	// Hence unregister all ports when the engine and by that this manager shut down. 
	// Alike the resource of ports (e.g. sockets) are always properly released.
	for (const FDMXInputPortSharedRef& InputPort : InputPorts)
	{
		InputPort->Unregister();
	}

	for (const FDMXOutputPortSharedRef& OutputPort : OutputPorts)
	{
		OutputPort->Unregister();
	}
}

FDMXPortManager& FDMXPortManager::Get()
{
#if UE_BUILD_DEBUG
	check(CurrentManager.IsValid());
#endif // UE_BUILD_DEBUG

	return *CurrentManager;
}

void FDMXPortManager::ClearBuffers()
{
	for (const FDMXInputPortSharedRef& InputPort : FDMXPortManager::Get().GetInputPorts())
	{
		InputPort->ClearBuffers();
	}

	for (const FDMXOutputPortSharedRef& OutputPort : FDMXPortManager::Get().GetOutputPorts())
	{
		OutputPort->ClearBuffers();
	}

	OnBuffersClearedDelegate.Broadcast();
}

FDMXInputPortSharedRef FDMXPortManager::GetInputPortFromConfigChecked(const FDMXInputPortConfig& InputPortConfig)
{
	// The config needs a valid guid
	check(InputPortConfig.GetPortGuid().IsValid());

	return FindInputPortByGuidChecked(InputPortConfig.GetPortGuid());
}

FDMXOutputPortSharedRef FDMXPortManager::GetOutputPortFromConfigChecked(const FDMXOutputPortConfig& OutputPortConfig)
{
	// The config needs a valid guid
	check(OutputPortConfig.GetPortGuid().IsValid());

	return FindOutputPortByGuidChecked(OutputPortConfig.GetPortGuid());
}

FDMXPortSharedPtr FDMXPortManager::FindPortByGuid(const FGuid& PortGuid) const
{
	const FDMXInputPortSharedRef* InputPortPtr =
		InputPorts.FindByPredicate([&PortGuid](const FDMXInputPortSharedRef& InputPort) {
		return InputPort->GetPortGuid() == PortGuid;
	});

	if (InputPortPtr)
	{
		return (*InputPortPtr);
	}

	const FDMXOutputPortSharedRef* OutputPortPtr =
		OutputPorts.FindByPredicate([&PortGuid](const FDMXOutputPortSharedRef& OutputPort) {
		return OutputPort->GetPortGuid() == PortGuid;
	});

	if (OutputPortPtr)
	{
		return (*OutputPortPtr);
	}

	return nullptr;
}

FDMXPortSharedRef FDMXPortManager::FindPortByGuidChecked(const FGuid& PortGuid) const
{
	const FDMXInputPortSharedRef* InputPortPtr =
		InputPorts.FindByPredicate([&PortGuid](const FDMXInputPortSharedRef& InputPort) {
		return InputPort->GetPortGuid() == PortGuid;
	});

	if (InputPortPtr)
	{
		return (*InputPortPtr);
	}

	const FDMXOutputPortSharedRef* OutputPortPtr =
		OutputPorts.FindByPredicate([&PortGuid](const FDMXOutputPortSharedRef& OutputPort) {
		return OutputPort->GetPortGuid() == PortGuid;
	});

	if (OutputPortPtr)
	{
		return (*OutputPortPtr);
	}

	// Check failed
	checkNoEntry();
	FDMXPortSharedPtr InvalidPtr;
	return InvalidPtr.ToSharedRef();
}

FDMXInputPortSharedPtr FDMXPortManager::FindInputPortByGuid(const FGuid& PortGuid) const
{
	const FDMXInputPortSharedRef* InputPortPtr =
		InputPorts.FindByPredicate([&PortGuid](const FDMXInputPortSharedRef& InputPort) {
		return InputPort->GetPortGuid() == PortGuid;
	});

	if (InputPortPtr)
	{
		return (*InputPortPtr);
	}

	return nullptr;
}

FDMXInputPortSharedRef FDMXPortManager::FindInputPortByGuidChecked(const FGuid& PortGuid) const
{
	const FDMXInputPortSharedRef* InputPortPtr =
		InputPorts.FindByPredicate([&PortGuid](const FDMXInputPortSharedRef& InputPort) {
		return InputPort->GetPortGuid() == PortGuid;
	});

	if (InputPortPtr)
	{
		return (*InputPortPtr);
	}

	// Check failed
	checkNoEntry();
	FDMXInputPortSharedPtr InvalidPtr;
	return InvalidPtr.ToSharedRef();
}

FDMXOutputPortSharedPtr FDMXPortManager::FindOutputPortByGuid(const FGuid& PortGuid) const
{
	const FDMXOutputPortSharedRef* OutputPortPtr =
		OutputPorts.FindByPredicate([&PortGuid](const FDMXOutputPortSharedRef& OutputPort) {
		return OutputPort->GetPortGuid() == PortGuid;
	});

	if (OutputPortPtr)
	{
		return (*OutputPortPtr);
	}

	return nullptr;
}

FDMXOutputPortSharedRef FDMXPortManager::FindOutputPortByGuidChecked(const FGuid& PortGuid) const
{
	const FDMXOutputPortSharedRef* OutputPortPtr =
		OutputPorts.FindByPredicate([&PortGuid](const FDMXOutputPortSharedRef& OutputPort) {
		return OutputPort->GetPortGuid() == PortGuid;
	});

	if (OutputPortPtr)
	{
		return (*OutputPortPtr);
	}

	// Check failed
	checkNoEntry();
	FDMXOutputPortSharedPtr InvalidPtr;
	return InvalidPtr.ToSharedRef();
}

void FDMXPortManager::UpdateFromProtocolSettings(bool bForceUpdateRegistrationWithProtocol)
{
	UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();

	// Remove Ports that no longer exist in settings
	TArray<FDMXInputPortSharedRef> CachedInputPorts = InputPorts;
	for (const FDMXInputPortSharedRef& InputPort : CachedInputPorts)
	{
		const FGuid& PortGuid = InputPort->GetPortGuid();
		const FDMXInputPortConfig* ExistingInputPortConfig = ProtocolSettings->InputPortConfigs.FindByPredicate([PortGuid](const FDMXInputPortConfig& InputPortConfig) {
			return InputPortConfig.GetPortGuid() == PortGuid;
			});

		if (!ExistingInputPortConfig &&
			PortGuidsFromProtocolSettings.Contains(PortGuid))
		{
			// Unregister the port to free a potential reference from its protocol
			InputPort->Unregister();

			InputPorts.Remove(InputPort);
			PortGuidsFromProtocolSettings.Remove(PortGuid);
		}
	}

	TArray<FDMXOutputPortSharedRef> CachedOutputPorts = OutputPorts;
	for (const FDMXOutputPortSharedRef& OutputPort : CachedOutputPorts)
	{
		const FGuid& PortGuid = OutputPort->GetPortGuid();
		const FDMXOutputPortConfig* ExistingOutputPortConfig = ProtocolSettings->OutputPortConfigs.FindByPredicate([PortGuid](const FDMXOutputPortConfig& OutputPortConfig) {
			return OutputPortConfig.GetPortGuid() == PortGuid;
		});

		if (!ExistingOutputPortConfig &&
			PortGuidsFromProtocolSettings.Contains(PortGuid))
		{
			// Unregister the port to free a potential references from its protocol
			OutputPort->Unregister();

			OutputPorts.Remove(OutputPort);
			PortGuidsFromProtocolSettings.Remove(PortGuid);
		}
	}

	// Add newly created ports and update existing ones
	for (FDMXInputPortConfig& InputPortConfig : ProtocolSettings->InputPortConfigs)
	{
		if (InputPortConfig.GetPortGuid().IsValid())
		{
			FDMXInputPortSharedRef InputPort = GetOrCreateInputPortFromConfig(InputPortConfig);

			InputPort->UpdateFromConfig(InputPortConfig, bForceUpdateRegistrationWithProtocol);

			PortGuidsFromProtocolSettings.AddUnique(InputPort->GetPortGuid());
		}
		else
		{
			UE_LOGF(LogDMXProtocol, Error, "Input Port '%ls' has no valid PortGUID and can no longer be used. If you changed the DefaultEngine.ini directly, please undo your changes. Otherwise please report the issue.", *InputPortConfig.GetPortName());
		}
	}

	for (FDMXOutputPortConfig& OutputPortConfig : ProtocolSettings->OutputPortConfigs)
	{
		if (OutputPortConfig.GetPortGuid().IsValid())
		{
			FDMXOutputPortSharedRef OutputPort = GetOrCreateOutputPortFromConfig(OutputPortConfig);

			OutputPort->UpdateFromConfig(OutputPortConfig, bForceUpdateRegistrationWithProtocol);

			PortGuidsFromProtocolSettings.AddUnique(OutputPort->GetPortGuid());
		}
		else
		{
			UE_LOGF(LogDMXProtocol, Error, "Output Port '%ls' has no valid GUID and can no longer be used. If you changed the DefaultEngine.ini directly, please undo your changes. Otherwise please report the issue.", *OutputPortConfig.GetPortName());
		}
	}

	OnPortsChanged.Broadcast();
}

void FDMXPortManager::SuspendProtocols()
{
	for (const FDMXInputPortSharedRef& InputPort : InputPorts)
	{
		InputPort->Unregister();
	}

	for (const FDMXOutputPortSharedRef& OutputPort : OutputPorts)
	{
		OutputPort->Unregister();
	}

	// Set this last to ensure thread safety of this function and AreProtocolsSuspended()
	bProtocolsSuspended = true;
}

FDMXInputPortSharedRef FDMXPortManager::GetOrCreateInputPortFromConfig(FDMXInputPortConfig& InputPortConfig)
{
	// The config needs a valid guid
	check(InputPortConfig.GetPortGuid().IsValid());

	FDMXInputPortSharedPtr ExistingPort = FindInputPortByGuid(InputPortConfig.GetPortGuid());
	if (ExistingPort.IsValid())
	{
		return ExistingPort.ToSharedRef();
	}

	FDMXInputPortSharedRef NewInputPort = FDMXInputPort::CreateFromConfig(InputPortConfig);
	InputPorts.Add(NewInputPort);

	OnPortsChanged.Broadcast();

	return NewInputPort;
}

void FDMXPortManager::RemoveInputPortChecked(const FGuid& PortGuid)
{
	FDMXInputPortSharedRef InputPort = FindInputPortByGuidChecked(PortGuid);
	InputPorts.Remove(InputPort);
}

FDMXOutputPortSharedRef FDMXPortManager::GetOrCreateOutputPortFromConfig(FDMXOutputPortConfig& OutputPortConfig)
{
	// The config needs a valid guid
	check(OutputPortConfig.GetPortGuid().IsValid());

	FDMXOutputPortSharedPtr ExistingPort = FindOutputPortByGuid(OutputPortConfig.GetPortGuid());
	if (ExistingPort.IsValid())
	{
		return ExistingPort.ToSharedRef();
	}

	FDMXOutputPortSharedRef NewOutputPort = FDMXOutputPort::CreateFromConfig(OutputPortConfig);
	OutputPorts.Add(NewOutputPort);

	OnPortsChanged.Broadcast();

	return NewOutputPort;
}

void FDMXPortManager::RemoveOutputPortChecked(const FGuid& PortGuid)
{
	FDMXOutputPortSharedRef OutputPort = FindOutputPortByGuidChecked(PortGuid);
	OutputPorts.Remove(OutputPort);
}

void FDMXPortManager::StartupManager()
{
	UE_LOGF(LogDMXProtocol, Verbose, "Starting up DMXPortManager");

	check(!CurrentManager.IsValid());
	CurrentManager = MakeUnique<FDMXPortManager>();
}

void FDMXPortManager::ShutdownManager()
{
	UE_LOGF(LogDMXProtocol, Verbose, "Shutting down DMXPortManager");

	CurrentManager.Reset();
}
