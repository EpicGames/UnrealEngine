// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/DMXRawListener.h"

#include "Async/Async.h"
#include "Async/Future.h"
#include "IO/DMXPort.h"


FDMXRawListener::FDMXRawListener(TSharedRef<FDMXPort, ESPMode::ThreadSafe> InOwnerPort)
	: OwnerPort(InOwnerPort)
	, ExternUniverseOffset(InOwnerPort->GetExternUniverseOffset())
	, bStopped(true)
#if UE_BUILD_DEBUG
	, ProducerObj(nullptr)
	, ConsumerObj(nullptr)
#endif // UE_BUILD_DEBUG
{}

FDMXRawListener::~FDMXRawListener()
{
	// Needs be stopped before releasing
	check(bStopped);

	if (TSharedPtr<FDMXPort, ESPMode::ThreadSafe> PinnedOwnerPort = OwnerPort.Pin())
	{
		PinnedOwnerPort->OnPortUpdated.RemoveAll(this);
	}
}

void FDMXRawListener::Start()
{
	if (TSharedPtr<FDMXPort, ESPMode::ThreadSafe> PinnedOwnerPort = OwnerPort.Pin())
	{
		PinnedOwnerPort->OnPortUpdated.AddSP(this, &FDMXRawListener::OnPortUpdated);
		PinnedOwnerPort->AddRawListener(AsShared());

		bStopped = false;
	}
}

void FDMXRawListener::Stop()
{
	// Without the RemoveRawListener method, the port would not know it had to free its shared pointer to this.
	// It may seem benefitial to hold a weak ref in the port, however assuming we may well have many
	// listeners, this would cause needless overhead when constantly copying of the weak ptr. 
	// This is less relevant for raw listeners, but kept for consistency with ticked universe 
	// listener and minimal performance benefits. 
	if (FDMXPortSharedPtr PinnedOwnerPort = OwnerPort.Pin())
	{
		PinnedOwnerPort->RemoveRawListener(AsShared());
	}

	bStopped = true;
}

void FDMXRawListener::EnqueueSignal(void* Producer, const FDMXSignalSharedRef& Signal)
{
#if UE_BUILD_DEBUG
	// Test single producer
	if (!ProducerObj)
	{
		ProducerObj = Producer;
	}
	FDMXPortSharedPtr PinnedOwnerPort = OwnerPort.Pin();
	checkf(PinnedOwnerPort.IsValid() && ProducerObj == PinnedOwnerPort.Get(), TEXT("More than one producer detected in FDMXRawListener::DequeueSignal."));
#endif // UE_BUILD_DEBUG

	RawBuffer.Enqueue(Signal);
}

bool FDMXRawListener::DequeueSignal(void* Consumer, FDMXSignalSharedPtr& OutSignal, int32& OutLocalUniverseID)
{
#if UE_BUILD_DEBUG
	// Test single consumer
	if (!ConsumerObj)
	{
		ConsumerObj = Consumer;
	}
	checkf(ConsumerObj == Consumer, TEXT("More than one consumer detected in FDMXRawListener::DequeueSignal."));
#endif // UE_BUILD_DEBUG

	if (RawBuffer.Dequeue(OutSignal))
	{
		OutLocalUniverseID = OutSignal->ExternUniverseID - ExternUniverseOffset;

		// Function comment states the OutSignal is always valid if true is returned
		// This is the case as only shared references are enqueued
		return true;
	}

	OutSignal = nullptr;
	OutLocalUniverseID = -1;
	return false;
}

void FDMXRawListener::ClearBuffer()
{
	if (IsInGameThread())
	{
		RawBuffer.Empty();
	}
	else
	{
		// The buffer has to be cleared on a consumer thread
		TPromise<bool> Promise;
		AsyncTask(ENamedThreads::GameThread, [&Promise, this]()
			{
				RawBuffer.Empty();
				Promise.SetValue(true);
			});

		// Lock the calling thread until the promise was fullfilled
		Promise.GetFuture().Get();
	}
}

void FDMXRawListener::OnPortUpdated()
{
	if (TSharedPtr<FDMXPort, ESPMode::ThreadSafe> PinnedOwnerPort = OwnerPort.Pin())
	{
		ExternUniverseOffset = PinnedOwnerPort->GetExternUniverseOffset();
	}
}
