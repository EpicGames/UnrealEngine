// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/DMXTrace.h"

#include "IO/DMXConflictMonitor.h"
#include "DMXStats.h"


DECLARE_CYCLE_STAT(TEXT("DMX Trace"), STAT_DMXTrace, STATGROUP_DMX);


#if WITH_EDITOR
namespace UE::DMX
{
	namespace Private::Trace
	{
		FDMXScopedSendDMXTrace::FDMXScopedSendDMXTrace(const FName& InUser)
			: User(InUser)
		{
			SCOPE_CYCLE_COUNTER(STAT_DMXTrace);

			if (FDMXConflictMonitor* ConflictMonitor = FDMXConflictMonitor::Get())
			{
				ConflictMonitor->TraceUser(User);
			}
		}

		FDMXScopedSendDMXTrace::~FDMXScopedSendDMXTrace()
		{
			if (FDMXConflictMonitor* ConflictMonitor = FDMXConflictMonitor::Get())
			{
				ConflictMonitor->PopTrace(User);
			}
		}
	}
}
#endif // WITH_EDITOR
