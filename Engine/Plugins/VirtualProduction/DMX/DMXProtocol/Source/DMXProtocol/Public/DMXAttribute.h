// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXNameListItem.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/Crc.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"

#include "DMXAttribute.generated.h"

namespace UE::DMX::Attribute
{
	class IDMXAttributeEditorContext;
}

USTRUCT()
struct DMXPROTOCOL_API FDMXAttribute
{
	GENERATED_BODY()

	/** Name of this Attribute, displayed on Attribute selectors */
	UPROPERTY(EditDefaultsOnly, Category = "DMX")
	FName Name;

	/**
	 * Keywords used when auto-mapping Fixture Functions from a GDTF file to
	 * match Fixture Functions to existing Attributes.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "DMX")
	FString Keywords;

	/** Comparison operator */
	inline bool operator==(const FDMXAttribute& Other) const
	{
		return Name.IsEqual(Other.Name) && Keywords.Equals(Other.Keywords);
	}
	inline bool operator!=(const FDMXAttribute& Other) const { return !(*this == Other); }

	TArray<FString> GetKeywords() const;

	// cleanup the list of keywords using commas and removing spaces/tabs
	void CleanupKeywords();
};

/** Unique hash from a DMX Attribute */
inline uint32 GetTypeHash(const FDMXAttribute& Attribute)
{
	return GetTypeHash(Attribute.Name);
}

USTRUCT(BlueprintType, Category = "DMX")
struct DMXPROTOCOL_API FDMXAttributeName
{
	GENERATED_BODY()

	using IDMXAttributeEditorContext = UE::DMX::Attribute::IDMXAttributeEditorContext;

public:
	FDMXAttributeName();

	/** Construct from an Attribute */
	explicit FDMXAttributeName(const FDMXAttribute& InAttribute);

	/** Construct from an Attribute name */
	FDMXAttributeName(const FName& NameAttribute);

	//~ FDMXNameListItem interface
	void SetFromName(const FName& InName);

	/** IsValid member accessor */
	bool IsValid() const { return !Name.IsNone(); }

	/** The Attribute Name */
	UPROPERTY(EditAnywhere, Category = "DMX")
	FName Name;

	/** Returns the predefined values */
	static TArray<FName> GetPredefinedValues();

	/** Custom serializer to maintain the upgrade path */
	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FDMXAttributeName>
	: public TStructOpsTypeTraitsBase2<FDMXAttributeName>
{
	enum
	{
		WithSerializer = true
	};
};

inline uint32 GetTypeHash(const FDMXAttributeName& DMXNameListItem)
{
	FString NameStr = DMXNameListItem.Name.ToString();

	return FCrc::MemCrc32(*NameStr, sizeof(TCHAR) * NameStr.Len());
}

inline bool operator==(const FDMXAttributeName& V1, const FDMXAttributeName& V2)
{
	return V1.Name.IsEqual(V2.Name);
}

inline bool operator!=(const FDMXAttributeName& V1, const FDMXAttributeName& V2)
{
	return !V1.Name.IsEqual(V2.Name);
}

UCLASS()
class UDMXAttributeNameConversions
	: public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToString (DMX Attribute)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static FString Conv_DMXAttributeToString(const FDMXAttributeName& InAttribute);

	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToName (DMX Attribute)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static FName Conv_DMXAttributeToName(const FDMXAttributeName& InAttribute);
};
