// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DMXProtocolTypes.h"

#include "Misc/ByteSwap.h"
#include "Templates/EnableIf.h"
#include "Templates/UnrealTypeTraits.h"


/** Common conversions for dmx */
class DMXPROTOCOL_API FDMXConversions
{
	//////////////////////////////////////////////////
	// TODO: All DMX conversions eventually should be moved here (e.g. from subsystem, fixture type) as statics, unit tested and optimized.
	// BP implementations in other classes should be forwarded here, native C++ implementations should be flagged deprecated to reduce code scattering.

public:
	/** Converts a uint32 to a byte array */
	template <typename T, typename U = typename TEnableIf<std::is_same_v<T, uint32>>::Type>
	static TArray<uint8> UnsignedInt32ToByteArray(T Value, EDMXFixtureSignalFormat SignalFormat, bool bLSBOrder)
	{
		const uint8 NumBytes = GetSizeOfSignalFormat(SignalFormat);

		// To avoid branching in the loop, we'll decide before it on which byte to start
		// and which direction to go, depending on the Function's endianness.
		const uint8 ByteIndexStep = bLSBOrder ? 1 : -1;
		uint8 OutByteIndex = bLSBOrder ? 0 : NumBytes - 1;

		TArray<uint8> Bytes;
		Bytes.AddUninitialized(NumBytes);
		for (uint8 ValueByte = 0; ValueByte < NumBytes; ++ValueByte)
		{
			Bytes[OutByteIndex] = (Value >> 8 * ValueByte) & 0xFF;
			OutByteIndex += ByteIndexStep;
		}

		return Bytes;
	}

	/** Converts a normalized value to a byte array. Normalized value has to be in the 0-1 range. Assumes max signal format is 24bit. */
	template <typename T, typename U = typename TEnableIf<std::is_same_v<T, float>>::Type>
	static TArray<uint8> NormalizedDMXValueToByteArray(T NormalizedValue, EDMXFixtureSignalFormat SignalFormat, bool bLSBOrder)
	{
		NormalizedValue = FMath::Clamp(NormalizedValue, 0.f, 1.f);
		uint32 Value = FMath::Floor<uint32>(TNumericLimits<uint32>::Max() * static_cast<double>(NormalizedValue));

		// Shift the value into signal format range
		Value = Value >> ((3 - static_cast<uint8>(SignalFormat)) * 8);

		return UnsignedInt32ToByteArray(Value, SignalFormat, bLSBOrder);
	}

	/** Returns the number of Bytes the Signal Format uses */
	static uint8 GetSizeOfSignalFormat(EDMXFixtureSignalFormat SignalFormat);

	/**
	 * Returns the Max Value the Data Type can take
	 *
	 * @param DataType		The Signal Format in quesion.
	 * @return				The Max Value of the Data Type, e.g. 255 for EDMXFixtureSignalFormat::E8Bit.
	 */
	static uint32 GetSignalFormatMaxValue(EDMXFixtureSignalFormat SignalFormat);

	/** Clamps the Value to be in value range of the Signal Format */
	static uint32 ClampValueBySignalFormat(uint32 Value, EDMXFixtureSignalFormat SignalFormat);
	template <typename T>
	static uint32 ClampValueBySignalFormat(T Value, EDMXFixtureSignalFormat SignalFormat) = delete;  // No implicit conversions

private:
	FDMXConversions() = delete;
};
