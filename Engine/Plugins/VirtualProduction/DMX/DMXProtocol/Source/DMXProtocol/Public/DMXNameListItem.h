// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DMXNameListItem.generated.h"

/**
 * Types that represent names in a SNameListPicker dropdown must
 * implement this macro in their type declaration body, just like it's done
 * with GENERATED_BODY().
 * In the type's cpp file, use IMPLEMENT_DMX_NAMELISTITEM_STATICS to define
 * the static variable and function.
 * 
 * @param bCanBeNone	Defines if the type can display a None value in
 *						its SNameListPicker dropdown.
 */
#define DECLARE_DMX_NAMELISTITEM_STATICS(CanBeNone) \
/** Whether SNameListPicker should display a None option for this type */\
static const bool bCanBeNone = CanBeNone;\
/** Delegate that should be called when the possible values for the type are changed */\
static FSimpleMulticastDelegate OnValuesChanged;\
/** Get the valid Names for the type */\
static TArray<FName> GetPossibleValues();\
 /** True if this object still represents one of the possible values */\
static bool IsValid(const FName& InName);

/**
 * Use this macro in the cpp file of the types that represent names in a
 * SNameListPicker dropdown to declare the static variables implementation
 * 
 * @param TypeName	The name of the type implementing the statics
 */
#define IMPLEMENT_DMX_NAMELISTITEM_STATICVARS(TypeName) \
FSimpleMulticastDelegate TypeName::OnValuesChanged;

/**
 * Implements the GetPossibleValues static function for a type that represents
 * names in a SNameListPicker dropdown.
 * 
 * Don't use ';' after the macro. Instead, add the body of GetPossibleValues
 * to it and return a TArray<FName> with all the possible values for the type.
 * Like so:
 * IMPLEMENT_DMX_NAMELISTITEM_GetAllValues(TypeName)
 * {
 *		TArray<FName> Names;
 *		// Add possible values to Names
 *		...
 *		return Names;
 * }
 * 
 * @param TypeName	The name of the type implementing the function
 */
#define IMPLEMENT_DMX_NAMELISTITEM_GetAllValues(TypeName) \
TArray<FName> TypeName::GetPossibleValues()

 /**
  * Implements the IsValid static function for a type that represents
  * names in a SNameListPicker dropdown.
  *
  * Don't use ';' after the macro. Instead, add the body of IsValid
  * to it and return a bool that is true if the InName parameter represents a
  * valid name for the type.
  * Like so:
  * IMPLEMENT_DMX_NAMELISTITEM_IsValid(TypeName)
  * {
  *		bool bIsNameValid = false;
  *		// Check the validity of InName
  *		...
  *		return bIsNameValid;
  * }
  *
  * @param TypeName	The name of the type implementing the function
  */
#define IMPLEMENT_DMX_NAMELISTITEM_IsValid(TypeName) \
bool TypeName::IsValid(const FName& InName)

/**
 * Types that represent names in a SNameListPicker dropdown must:
 * - inherit this struct and override the virtual functions.
 * - use the DMX_NAMELISTITEM_STATICS macros above to declare the expected statics.
 */
struct UE_DEPRECATED(5.1, "Removed to overcome restrictions of the generic base struct. Ustructs that have a Name property should declare it themselves instead.") FDMXNameListItem;
USTRUCT()
struct DMXPROTOCOL_API FDMXNameListItem
{
	GENERATED_BODY()

	/** The value of no item selected. */
	static const FName None;

	virtual ~FDMXNameListItem() = default;

	/** Label of the item this struct represents */
	UPROPERTY(EditAnywhere, Category = "DMX")
	FName Name;

	/** The name representation of this object */
	virtual FName GetName() const { return Name; };

	/** Set this object from a FName */
	virtual void SetFromName(const FName& InName) { Name = InName; }

	/** Whether the current value is None */
	virtual bool IsNone() const { return Name.IsNone(); }

	/** Set this object to represent a None value */
	virtual void SetToNone() { Name = NAME_None; }
};
