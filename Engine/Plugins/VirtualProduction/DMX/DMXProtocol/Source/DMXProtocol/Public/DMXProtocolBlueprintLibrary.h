// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/DMXInputPortReference.h"
#include "IO/DMXOutputPortReference.h"

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "DMXProtocolBlueprintLibrary.generated.h"


UCLASS(meta = (ScriptName = "DMXRuntimeLibrary"))
class DMXPROTOCOL_API UDMXProtocolBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Sets if DMX is sent to the network
	 * @param bSendDMXEnabled		If true, sends DMX packets to the output ports, else ignores all send calls globally.
	 * @param bAffectEditor			If true, affects the editor.
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	static void SetSendDMXEnabled(bool bSendDMXEnabled = true, bool bAffectEditor = false);

	/**
	 * Returns whether send DMX to the network is enabled globally.
	 * @return		If true, DMX is sent to the Network
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	static bool IsSendDMXEnabled();

	/**
	 * Sets if DMX is received from the network
	 * @param bReceiveDMXEnabled	If true, receives inbound DMX packets on the input ports, else ignores them, globally.
	 * @param bAffectEditor			If true, affects the editor. 
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	static void SetReceiveDMXEnabled(bool bReceiveDMXEnabled = true, bool bAffectEditor = false);

	/**
	 * Returns whether Receive DMX from the network is enabled globally.
	 * @return		If true, DMX is received from the Network
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	static bool IsReceiveDMXEnabled();

	/**
	 * Returns the IP addresses of the network interface cards available to the system.
	 * @return		The Network Interface Card IP Addresses
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	static TArray<FString> GetLocalDMXNetworkInterfaceCardIPs();

	/**
	 * Sets the Device Address of the Output Port. For networking Protocols that's the IP Adress of the network interface card.
	 * @param InputPort			The Input Port for which the Device Address should be set
	 * @param DeviceAddress		The Device Address the Input Port should use
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	static void	SetDMXInputPortDeviceAddress(FDMXInputPortReference InputPort, const FString& DeviceAddress);

	/**
	 * Sets the Device Address of the Output Port. For networking Protocols that's the IP Adress of the network interface card.
	 * @param OutputPort		The Output Port for which the Device Address should be set
	 * @param DeviceAddress		The Device Address the Output Port should use 
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	static void	SetDMXOutputPortDeviceAddress(FDMXOutputPortReference OutputPort, const FString& DeviceAddress);

	/**
	 * Sets the Destination Address Address of the Output Port. For networking Protocols that's the Unicast IP Adress. Not required for Multicast and Broadcast.
	 * @param PortName				The Output Port for which the Unicast IP Address should be set
	 * @param DestinationAddress	The Destination Address the Output Port should use 
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	static void	SetDMXOutputPortDestinationAddresses(FDMXOutputPortReference OutputPort, const TArray<FString>& DestinationAddresses);

	/**
	 * Sets the Destination Address Address of the Output Port. For networking Protocols that's the Unicast IP Adress. Not required for Multicast and Broadcast.
	 * @param PortName				The Output Port for which the Unicast IP Address should be set
	 * @param DestinationAddress	The Destination Address the Output Port should use 
	 */
	UE_DEPRECATED(5.0, "Output Ports now support many destination addresses. Please use UDMXProtocolBlueprintLibrary::SetDMXOutputPortDestinationAddresses instead")
	UFUNCTION(BlueprintCallable, Category = "DMX", meta = (DeprecatedFunction, DeprecationMessage = "Deprecated 5.0. Output Ports now support many Destination Addresses. Please use SetDMXOutputPortDestinationAddresses instead."))
	static void	SetDMXOutputPortDestinationAddress(FDMXOutputPortReference OutputPort, const FString& DestinationAddress);
};
