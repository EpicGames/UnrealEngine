// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DMXProtocolConstants.h"

class FDMXInputPort;
class FDMXOutputPort;
class IDMXProtocol;
class FDMXPort;
class FDMXSignal;


// Protocol
using IDMXProtocolPtr = TSharedPtr<IDMXProtocol, ESPMode::ThreadSafe>;
using IDMXProtocolPtrWeak = TWeakPtr<IDMXProtocol, ESPMode::ThreadSafe>;

// Ports
using FDMXPortSharedPtr = TSharedPtr<class FDMXPort, ESPMode::ThreadSafe>;
using FDMXPortSharedRef = TSharedRef<class FDMXPort, ESPMode::ThreadSafe>;

using FDMXInputPortSharedPtr = TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>;
using FDMXInputPortSharedRef = TSharedRef<FDMXInputPort, ESPMode::ThreadSafe>;

using FDMXOutputPortSharedPtr = TSharedPtr<FDMXOutputPort, ESPMode::ThreadSafe>;
using FDMXOutputPortSharedRef = TSharedRef<FDMXOutputPort, ESPMode::ThreadSafe>;

// Signal
using FDMXSignalSharedPtr = TSharedPtr<FDMXSignal, ESPMode::ThreadSafe>;
using FDMXSignalSharedRef = TSharedRef<FDMXSignal, ESPMode::ThreadSafe>;
