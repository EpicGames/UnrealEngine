// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum 
{
  DMX_UNIVERSE_SIZE = 512,
  DMX_MAX_CHANNEL_VALUE = 255,
  RDM_UID_WIDTH = 6
};

#define DMX_RATE 44
#define DMX_MAX_ADDRESS 512
#define DMX_MAX_VALUE 255
#define DMX_MAX_FUNCTION_SIZE ((int32)sizeof(int32))
#define DMX_MAX_UNIVERSE 63999
#define DMX_K2_CATEGORY_NAME "DMX"
