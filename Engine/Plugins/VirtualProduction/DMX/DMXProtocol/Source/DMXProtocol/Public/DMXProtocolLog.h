// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


DECLARE_LOG_CATEGORY_CLASS(LogDMXProtocol, Log, All);

#define UE_LOG_DMXPROTOCOL(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogDMXProtocol, Verbosity, "DMX: %ls", *FString::Printf(Format, ##__VA_ARGS__)); \
}

#define UE_CLOG_DMXPROTOCOL(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOGF(Conditional, LogDMXProtocol, Verbosity, "DMX: %ls", *FString::Printf(Format, ##__VA_ARGS__)); \
}
