// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

#define REGISTER_DMX_ARCHIVE(ProtocolDMXPacket) \
	FArchive & operator<<(FArchive & Ar, ProtocolDMXPacket&Packet) \
	{ \
		Packet.Serialize(Ar); \
		return Ar; \
	}
