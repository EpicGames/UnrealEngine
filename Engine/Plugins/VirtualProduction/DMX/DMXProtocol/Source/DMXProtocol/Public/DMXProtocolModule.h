// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"

#include "CoreMinimal.h"
#include "ModuleDescriptor.h"
#include "Modules/ModuleInterface.h"


class IDMXProtocolFactory;

struct FDMXProtocolRegistrationParams
{
	/** The name to use for the protocol (also used in UIs) */
	FName ProtocolName;

	/** The factory used to create the protocol */
	IDMXProtocolFactory* ProtocolFactory = nullptr;
};



/** 
 * Implements the Protocol Module, that enables specific Protocol implementations.
 * 
 * For use in Engine, see comments in DMXPortManager.h
 * For protocol development, see comments in IDMXProtocol.h
 */
class DMXPROTOCOL_API FDMXProtocolModule 
	: public IModuleInterface
{

public:
	/** 
	 * Event Broadcast when protocols need to register with the Protocol Module. 
	 * The event is broadcast at the end of the end of the PreDefault loading phase, so all protocol implementations should be implemented as PreDefault loading phase modules.
	 * 
	 * See DMXProtocolArtNet for an example of a Protocol implementation.
	 */
	DECLARE_EVENT_OneParam(FDMXProtocolModule, FDMXOnRequestProtocolRegistrationEvent, TArray<FDMXProtocolRegistrationParams>& /** InOutProtocolRegistrationParamsArray */);
	static FDMXOnRequestProtocolRegistrationEvent& GetOnRequestProtocolRegistration();

	/** Event broadcast so other Plugins can block specific protocols from registering. */
	DECLARE_EVENT_OneParam(FDMXProtocolModule, FDMXOnRequestProtocolBlocklistEvent, TArray<FName>& /** InOutBlocklistedProtocols */);
	static FDMXOnRequestProtocolBlocklistEvent& GetOnRequestProtocolBlocklist();

	UE_DEPRECATED(4.27, "Please use the OnRequestProtocolRegistration event instead.")
	void RegisterProtocol(const FName& ProtocolName, IDMXProtocolFactory* Factory);

	void UnregisterProtocol(const FName& ProtocolName);

	/** Get the instance of this module. */
	static FDMXProtocolModule& Get();

	/**
	 * If protocol exists return the pointer otherwise it create a new protocol first and then return the pointer.
	 * @param  InProtocolName Name of the requested protocol
	 * @return Return the pointer to protocol.
	 */
	virtual IDMXProtocolPtr GetProtocol(const FName InProtocolName = NAME_None);
	
	/**  Get the reference to all protocol factories map */
	const TMap<FName, IDMXProtocolFactory*>& GetProtocolFactories() const;

	/**  Get the reference to all protocols map */
	const TMap<FName, IDMXProtocolPtr>& GetProtocols() const;

	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface implementation

	static const FName DefaultProtocolArtNetName;
	static const FName DefaultProtocolSACNName;

private:
	/** Called after each loading phase during startup */
	void OnPluginLoadingPhaseComplete(ELoadingPhase::Type LoadingPhase, bool bPhaseSuccessful);

	void ShutdownDMXProtocol(const FName& ProtocolName);
	void ShutdownAllDMXProtocols();

private:
	static FDMXOnRequestProtocolRegistrationEvent OnRequestProtocolRegistrationEvent;
	static FDMXOnRequestProtocolBlocklistEvent OnRequestProtocolBlocklistEvent;

	TMap<FName, IDMXProtocolFactory*> DMXProtocolFactories;
	TMap<FName, IDMXProtocolPtr> DMXProtocols;
};
