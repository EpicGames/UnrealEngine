// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolObjectVersion.h"

#include "Serialization/CustomVersion.h"

const FGuid FDMXProtocolObjectVersion::GUID(0x94662356, 0x4DAE92AC, 0xABF8BE98, 0x19FE8C2F);

// Register the custom version with core
FCustomVersionRegistration GRegisterDMXProtocolCustomVersion(FDMXProtocolObjectVersion::GUID, FDMXProtocolObjectVersion::LatestVersion, TEXT("DMXProtocolObjectVersion"));
