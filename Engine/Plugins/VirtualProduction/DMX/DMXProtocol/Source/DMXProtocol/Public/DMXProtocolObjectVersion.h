// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes to DMX Protocol Objects
struct FDMXProtocolObjectVersion
{
	enum Type
	{
		// Roughly corresponds to 4.27
		BeforeCustomVersionWasAdded = 0,

		// Update Ports to allow for an array of Unicast Addresses instead of just one
		OutputPortSupportsManyUnicastAddresses,

		// 5.5 fix UE-224455, variable default for FDXMAttribute::Name
		FixAttributeNameDefaultValue,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FDMXProtocolObjectVersion() {}
};
