// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DMXProtocolTypes.h"
#include "DMXAttribute.h"
#include "IO/DMXInputPortConfig.h"
#include "IO/DMXOutputPortConfig.h"
#include "Misc/Optional.h"
#include "UObject/Object.h"

#include "DMXProtocolSettings.generated.h"


/**  
 * DMX Project Settings. 
 * 
 * Note: To handle Port changes in code please refer to FDMXPortManager.
 */
UCLASS(Config = Engine, DefaultConfig, AutoExpandCategories = ("DMX|Communication Settings"), Meta = (DisplayName = "DMX"))
class DMXPROTOCOL_API UDMXProtocolSettings 
	: public UObject
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FDMXOnSendDMXEnabled, bool /** bEnabled */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FDMXOnReceiveDMXEnabled, bool /** bEnabled */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FDMXOnAllFixturePatchesReceiveDMXInEditorEnabled, bool /** bEnabled */);

public:
	GENERATED_BODY()

public:
	UDMXProtocolSettings();

	// ~Begin UObject Interface
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
#endif // WITH_EDITOR
	// ~End UObject Interface

	/** Returns the input port matching the predicate, or nullptr if it cannot be found */
	template <typename Predicate>
	FDMXInputPortConfig* FindInputPortConfig(Predicate Pred)
	{
		return InputPortConfigs.FindByPredicate(Pred);
	}

	/** Returns the input port matching the predicate, or nullptr if it cannot be found */
	template <typename Predicate>
	FDMXOutputPortConfig* FindOutputPortConfig(Predicate Pred)
	{
		return OutputPortConfigs.FindByPredicate(Pred);
	}

	/** DMX Input Port Configs */
	UPROPERTY(Config, EditAnywhere, Category = "DMX|Communication Settings", Meta = (DisplayName = "Input Ports"))
	TArray<FDMXInputPortConfig> InputPortConfigs;

	/** DMX Output Port Configs */
	UPROPERTY(Config, EditAnywhere, Category = "DMX|Communication Settings", Meta = (DisplayName = "Output Ports"))
	TArray<FDMXOutputPortConfig> OutputPortConfigs;
		
	/** Rate at which DMX is sent, in Hz from 1 to 1000. 44Hz is recommended. */
	UPROPERTY(Config, EditAnywhere, Category = "DMX|Communication Settings", Meta = (ClampMin = "1", ClampMax = "1000"), Meta = (DisplayName = "DMX Send Rate"))
	uint32 SendingRefreshRate;

	/** Rate at which DMX is received, in Hz from 1 to 1000. 44Hz is recommended */
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "ReceivingRefreshRate is deprecated without replacement. It would prevent from precise timestamps on the receivers."))
	uint32 ReceivingRefreshRate_DEPRECATED;

	/** Fixture Categories ENum */
	UPROPERTY(Config, EditAnywhere, Category = "DMX|Fixture Settings", Meta = (DisplayName = "Fixture Categories"))
	TSet<FName> FixtureCategories;

	/** Common names to map Fixture Functions to and access them easily on Blueprints */
	UPROPERTY(Config, EditAnywhere, Category = "DMX|Fixture Settings", Meta = (DisplayName = "Fixture Attributes"))
	TSet<FDMXAttribute> Attributes;

	/** Returns whether send DMX is currently enabled, considering runtime override */
	bool IsSendDMXEnabled() const { return bOverrideSendDMXEnabled; }

	/** Overrides if send DMX is enabled at runtime */
	void OverrideSendDMXEnabled(bool bEnabled);

	/** Returns whether receive DMX is currently enabled, considering runtime override */
	bool IsReceiveDMXEnabled() const { return bOverrideReceiveDMXEnabled; }

	/** Overrides if send DMX is enabled at runtime */
	void OverrideReceiveDMXEnabled(bool bEnabled);

	/** Returns an automatic/unique name for an InputPort */
	FString GetUniqueInputPortName() const;

	/** Returns an automatic/unique name for an OutputPort */
	FString GetUniqueOutputPortName() const;

	/** Gets a delegate broadcast when the Sending Refresh Rate (the DMX send rate) changed */
	FSimpleMulticastDelegate& GetOnSendingRefresRateChanged()
	{
		return OnSendingRefresRateChanged;
	}

	/** Gets a delegate broadcast when send DMX is enabled or disabled */
	FDMXOnSendDMXEnabled& GetOnSetSendDMXEnabled()
	{
		return OnSetSendDMXEnabledDelegate;
	}

	/** Gets a delegate broadcast when receive DMX is enabled or disabled */
	FDMXOnSendDMXEnabled& GetOnSetReceiveDMXEnabled()
	{
		return OnSetReceiveDMXEnabledDelegate;
	}

	/** Gets a delegate broadcast when default attributes were changed */
	FSimpleMulticastDelegate& GetOnDefaultAttributesChanged()
	{
		return OnDefaultAttributesChanged;
	}

	/** Gets a delegate broadcast when default fixture categories were changed */
	FSimpleMulticastDelegate& GetOnDefaultFixtureCategoriesChanged()
	{
		return OnDefaultFixtureCategoriesChanged;
	}

#if WITH_EDITOR
	/** Returns true if Fixture Patches should receive DMX in Editor */
	bool ShouldAllFixturePatchesReceiveDMXInEditor() const { return bAllFixturePatchesReceiveDMXInEditor; }

	/** Gets a Delegate Broadcast when receive DMX is enabled or disabled */
	FDMXOnAllFixturePatchesReceiveDMXInEditorEnabled& GetOnAllFixturePatchesReceiveDMXInEditorEnabled()
	{
		return OnAllFixturePatchesReceiveDMXInEditorEnabled;
	}
#endif
	
	/** DEPRECATED 5.1 */
	UE_DEPRECATED(5.1, "The OnSetSendDMXEnabled delegate is no longer directly accessible. Please use UDMXProtocolSettings::GetOnSetDMXEnabled() instead to access the delegate")
	FDMXOnSendDMXEnabled OnSetSendDMXEnabled;

	/** DEPRECATED 5.1 */
	UE_DEPRECATED(5.1, "The OnSetReceiveDMXEnabled delegate is no longer directly accessible. Please use UDMXProtocolSettings::GetOnSetReceiveDMXEnabled() instead to access the delegate")
	FDMXOnReceiveDMXEnabled OnSetReceiveDMXEnabled;

	/** Returns the property name of the bDefaultSendDMXEnabled Property */
	FORCEINLINE static FName GetDefaultSendDMXEnabledPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, bDefaultSendDMXEnabled); }

	/** Returns the property name of the bDefaultReceiveDMXEnabled Property */
	FORCEINLINE static FName GetDefaultReceiveDMXEnabledPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, bDefaultReceiveDMXEnabled); }

#if WITH_EDITOR	
	/** Returns the property name of the bDMXComponentRecievesDMXInEditor Property */
	FORCEINLINE static FName GetAllFixturePatchesReceiveDMXInEditorPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXProtocolSettings, bAllFixturePatchesReceiveDMXInEditor); }
#endif

private:
	/** Whether DMX is sent to the network. Recalled whenever editor or game starts.  */
	UPROPERTY(Config, EditAnywhere, Category = "DMX|Communication Settings", Meta = (AllowPrivateAccess = true, DisplayName = "Send DMX by default"))
	bool bDefaultSendDMXEnabled = true;

	/** Whether DMX is received from the network. Recalled whenever editor or game starts. */
	UPROPERTY(Config, EditAnywhere, Category = "DMX|Communication Settings", Meta = (AllowPrivateAccess = true, DisplayName = "Receive DMX by default"))
	bool bDefaultReceiveDMXEnabled = true;

	/** Overrides the default bDefaultSendDMXEnabled value at runtime */
	bool bOverrideSendDMXEnabled = true;

	/** Overrides the default bDefaultReceiveDMXEnabled value at runtime */
	bool bOverrideReceiveDMXEnabled = true;

	/** Broadcast the Sending Refresh Rate (the DMX send rate) changed */
	FSimpleMulticastDelegate OnSendingRefresRateChanged;

	/** Broadcast when send DMX is enabled or disabled */
	FDMXOnSendDMXEnabled OnSetSendDMXEnabledDelegate;

	/** Broadcast when receive DMX is enabled or disabled */
	FDMXOnReceiveDMXEnabled OnSetReceiveDMXEnabledDelegate;

	/** Broadcast when default attributets changed */
	FSimpleMulticastDelegate OnDefaultAttributesChanged;

	/** Broadcast when default fixture categories changed */
	FSimpleMulticastDelegate OnDefaultFixtureCategoriesChanged;

#if WITH_EDITORONLY_DATA
	/** If true, all fixture patches receive DMX in Editor. This overrides the fixture patches 'Receive DMX In Editor' property. */
	UPROPERTY(Config, EditAnywhere, Category = "DMX|Communication Settings", Meta = (DisplayName = "All Fixture Patches receive DMX in Editor"))
	bool bAllFixturePatchesReceiveDMXInEditor = true;

	/** Broadcast when Fixture Patches recieve DMX in Editor is enabled or disabled */
	FDMXOnAllFixturePatchesReceiveDMXInEditorEnabled OnAllFixturePatchesReceiveDMXInEditorEnabled;
#endif 

	///////////////////
	// DEPRECATED 4.27
public:
	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Now in Port Configs to support many NICs")
	UPROPERTY(Config, Meta = (DeprecatedProperty, DeprecationMessage = "InterfaceIPAddress is deprecated. Use Ports instead."))
	FString InterfaceIPAddress_DEPRECATED;

	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Now in Port Configs to support many NICs")
	UPROPERTY(Config, Meta = (DeprecatedProperty, DeprecationMessage = "GlobalArtNetUniverseOffset is deprecated. Use Ports instead."))
	int32 GlobalArtNetUniverseOffset_DEPRECATED;

	/** DEPRECATED 4.27 */
	UE_DEPRECATED(4.27, "Now in Port Configs to support many NICs")
	UPROPERTY(Config, Meta = (DeprecatedProperty, DeprecationMessage = "GlobalSACNUniverseOffset is deprecated. Use Ports instead."))
	int32 GlobalSACNUniverseOffset_DEPRECATED;
};
