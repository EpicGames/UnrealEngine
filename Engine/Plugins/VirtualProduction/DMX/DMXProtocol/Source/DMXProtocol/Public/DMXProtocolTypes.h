// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"
#include "DMXProtocolConstants.h"
#include "DMXProtocolLog.h"
#include "DMXProtocolMacros.h"

#include "Dom/JsonObject.h"
#include "HAL/CriticalSection.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/App.h"
#include "Templates/Atomic.h"

#include "DMXProtocolTypes.generated.h"


/** Type of network communication */
UENUM()
enum class EDMXCommunicationType : uint8
{
	Broadcast,
	Unicast,
	Multicast,
	InternalOnly	// For protocols that do not expose this as a selection to users
};

/** A single, generic DMX signal. One universe of raw DMX data received */
class FDMXSignal
	: public TSharedFromThis<FDMXSignal, ESPMode::ThreadSafe>
{
public:
	FDMXSignal()
		: Timestamp()
		, ExternUniverseID(0)
		, Priority(0)
		, ChannelData()
	{
		ChannelData.AddZeroed(DMX_UNIVERSE_SIZE);
	}

	FDMXSignal(const double InTimestamp, const int32 InUniverseID, const int32 InPriority, const TArray<uint8>& InChannelData)
		: Timestamp(InTimestamp)
		, ExternUniverseID(InUniverseID)
		, Priority(InPriority)
		, ChannelData(InChannelData)
	{}

	FDMXSignal(const double InTimestamp, const int32 InUniverseID, const int32 InPriority, TArray<uint8>&& InChannelData)
		: Timestamp(InTimestamp)
		, ExternUniverseID(InUniverseID)
		, ChannelData(InChannelData)
	{}

	void Serialize(FArchive& Ar)
	{
		Ar << Timestamp;
		Ar << ExternUniverseID;
		Ar << Priority;
		Ar << ChannelData;
	}

	double Timestamp;

	int32 ExternUniverseID;

	int32 Priority = 100;

	TArray<uint8> ChannelData;
};

/** Result when sending a DMX packet */
UENUM(BlueprintType, Category = "DMX")
enum class EDMXSendResult : uint8
{
	Success UMETA(DisplayName = "Successfully sent"),
	ErrorGetUniverse UMETA(DisplayName = "Error Get Universe"),
	ErrorSetBuffer UMETA(DisplayName = "Error Set Buffer"),
	ErrorSizeBuffer UMETA(DisplayName = "Error Size Buffer"),
	ErrorEnqueuePackage UMETA(DisplayName = "Error Enqueue Package"),
	ErrorNoSenderInterface UMETA(DisplayName = "Error No Sending Interface")
};

UENUM(BlueprintType, Category = "DMX")
enum class EDMXFixtureSignalFormat : uint8
{
	/** Uses 1 channel (byte). Range: 0 to 255 */
	E8Bit  = 0 	UMETA(DisplayName = "8 Bit"),
	/** Uses 2 channels (bytes). Range: 0 to 65.535 */
	E16Bit = 1	UMETA(DisplayName = "16 Bit"),
	/** Uses 3 channels (bytes). Range: 0 to 16.777.215 */
	E24Bit = 2	UMETA(DisplayName = "24 Bit"),
	/** Uses 4 channels (bytes). Range: 0 to 4.294.967.295 */
	E32Bit = 3	UMETA(DisplayName = "32 Bit"),
};

/** A DMX protocol as a name that can be displayed in UI. The protocol is directly accessible via GetProtocol */
USTRUCT(BlueprintType, Category = "DMX")
struct DMXPROTOCOL_API FDMXProtocolName
{
public:
	GENERATED_BODY()

	FDMXProtocolName();

	/** Construct from a protocol name */
	explicit FDMXProtocolName(const FName& InName);

	/** Construct from a protocol */
	FDMXProtocolName(IDMXProtocolPtr InProtocol);

	/** Returns the Protocol this name represents */
	IDMXProtocolPtr GetProtocol() const;

	/** IsValid member accessor */
	bool IsValid() const { return !Name.IsNone(); }

	//~ FName operators
	operator FName&() { return Name; }
	operator const FName&() const { return Name; }

	//~ DMXProtocol operators
	operator IDMXProtocolPtr() { return GetProtocol(); }
	operator const IDMXProtocolPtr() const { return GetProtocol(); }

	/** Bool (is valid) operator */
	UE_DEPRECATED(5.1, "Please use IsValid() instead.")
	operator bool() const { return !Name.IsNone(); }

	/** The Protocol Name */
	UPROPERTY(EditAnywhere, Category = "DMX")
	FName Name;

	//~ Comparison operators
	bool operator==(const FDMXProtocolName& Other) const { return Name == Other.Name; }
	bool operator!=(const FDMXProtocolName& Other) const { return !Name.IsEqual(Other.Name); }
	bool operator==(const IDMXProtocolPtr& Other) const { return GetProtocol().Get() == Other.Get(); }
	bool operator!=(const IDMXProtocolPtr& Other) const { return GetProtocol().Get() != Other.Get(); }
	bool operator==(const FName& Other) const { return Name == Other; }
	bool operator!=(const FName& Other) const { return !Name.IsEqual(Other); }

	//////////////////////////////////////////////////////////////////
	// Deprecated members originating from deprecated FDMXNameListItem
	UE_DEPRECATED(5.1, "Instead please use IDMXProtocol::GetProtocolNames")
	static TArray<FName> GetPossibleValues();

	// Deprecated without replacement. Always false.
	UE_DEPRECATED(5.1, "Obsolete. The value never can and never should be none in any case.")
	static const bool bCanBeNone;

	// Deprecated without replacement. Protocols cannot be enabled or disabled after engine startup.
	UE_DEPRECATED(5.1, "Obsolete. Protocols cannot be enabled or disabled after engine startup.")
	static FSimpleMulticastDelegate OnValuesChanged;
};

/** Category of a fixture */
USTRUCT(BlueprintType, Category = "DMX")
struct DMXPROTOCOL_API FDMXFixtureCategory
{
public:
	GENERATED_BODY()

	static FName GetFirstValue();

	FDMXFixtureCategory();

	explicit FDMXFixtureCategory(const FName& InName);
	
	/** The Protocol Name */
	UPROPERTY(EditAnywhere, Category = "DMX")
	FName Name;

	/** Returns the predefined values */
	static TArray<FName> GetPredefinedValues();

	//~ FName operators
	operator FName&() { return Name; }
	operator const FName&() const { return Name; }

	/** Bool (is valid) operator */
	UE_DEPRECATED(5.1, "Please use IsValid() instead.")
	operator bool() const { return !Name.IsNone(); }

	//~ Comparison operators
	bool operator==(const FDMXFixtureCategory& Other) const { return Name == Other.Name; }
	bool operator==(const FName& Other) const { return Name == Other; }

	//////////////////////////////////////////////////////////////////
	// Deprecated members originating from deprecated FDMXNameListItem
	UE_DEPRECATED(5.1, "Please use GetPredefinedValues() instead")
	static TArray<FName> GetPossibleValues();

	// Deprecated without replacement. Always false
	UE_DEPRECATED(5.1, "Obsolete. The value never can and never should be none in any case.")
	static const bool bCanBeNone;

	// Deprecated without replacement. Protocols cannot be enabled or disabled after engine startup.
	UE_DEPRECATED(5.1, "Obsolete. To listen to default fixture category changes, please refer to UDMXProtocolSettings::GetOnDefaultFixtureCategoriesChanged().")
	static FSimpleMulticastDelegate OnValuesChanged;
};

UCLASS()
class UDMXNameContainersConversions
	: public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToString (DMX Protocol Name)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static FString Conv_DMXProtocolNameToString(const FDMXProtocolName& InProtocolName);

	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToName (DMX Protocol Name)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static FName Conv_DMXProtocolNameToName(const FDMXProtocolName& InProtocolName);

	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToString (DMX Fixture Category)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static FString Conv_DMXFixtureCategoryToString(const FDMXFixtureCategory& InFixtureCategory);

	UFUNCTION(BlueprintPure, meta = (DisplayName = "ToName (DMX Fixture Category)", CompactNodeTitle = "->", BlueprintAutocast), Category = "Utilities|DMX")
	static FName Conv_DMXFixtureCategoryToName(const FDMXFixtureCategory& InFixtureCategory);
};

