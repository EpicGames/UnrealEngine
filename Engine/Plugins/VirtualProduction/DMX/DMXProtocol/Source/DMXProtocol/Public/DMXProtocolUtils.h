// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FInternetAddr;


class DMXPROTOCOL_API FDMXProtocolUtils
{
public:
	/** Returns an array of local network interfaces */
	static TArray<TSharedPtr<FString>> GetLocalNetworkInterfaceCardIPs();

	/** 
	 * Creates an Internet Address 
	 * 
	 * @param IPAddress	The IP Address to use
	 * @param Port		The Port to use
	 * 
	 * @return			The Internet Address or nullptr if IPAddress and Port don't yield a valid Internet Address
	 */
	static TSharedPtr<FInternetAddr> CreateInternetAddr(const FString& IPAddress, int32 Port);

	/** Tries to find a local network interface card IP that matches an IP Address with wildcards */
	static bool FindLocalNetworkInterfaceCardIPAddress(const FString& InIPAddressWithWildcards, FString& OutLocalNetworkInterfaceCardIPAddress);

	/**
	 * Generates a unique name given a base one and a list of existing ones, by appending an index to
	 * existing names. If InBaseName is an empty String, it returns "Default name".
	 */
	static FString GenerateUniqueNameFromExisting(const TSet<FString>& InExistingNames, const FString& InBaseName);

	/**
	 * Utility to separate a name from an index at the end.
	 * @param InString	The string to be separated.
	 * @param OutName	The string without an index at the end. White spaces and '_' are also removed.
	 * @param OutIndex	Index that was separated from the name. If there was none, it's zero.
	 *					Check the return value to know if there was an index on InString.
	 * @return True if there was an index on InString.
	 */
	static bool GetNameAndIndexFromString(const FString& InString, FString& OutName, int32& OutIndex);

	// can't instantiate this class
	FDMXProtocolUtils() = delete;
};
