// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("DMX"), STATGROUP_DMX, STATCAT_Advanced);
