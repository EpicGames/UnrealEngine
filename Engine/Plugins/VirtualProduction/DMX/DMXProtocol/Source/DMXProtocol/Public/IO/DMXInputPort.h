// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPort.h"

#include "DMXProtocolCommon.h"

#include "CoreMinimal.h"
#include "Tickable.h"
#include "Containers/Queue.h" 
#include "HAL/CriticalSection.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"

struct FDMXInputPortConfig;
class FDMXPortManager;
class FDMXRawListener;

enum class EDMXPortPriorityStrategy : uint8;

/**  
 * Higher level abstraction of a DMX input hiding networking specific and protocol specific complexity from the game. 
 *
 * To input DMX into your objects, refer to DMXRawListener and DMXTickedUniverseListener.
 *
 * Can only be constructed via DMXPortManger, see FDMXPortManager::CreateInputPort and FDMXPortManager::CreateInputPortFromConfig
 */
class DMXPROTOCOL_API FDMXInputPort
	: public FDMXPort
	, public FTickableGameObject
{
	// Friend DMXPortManager so it can create instances and unregister void instances
	friend FDMXPortManager;

	// Friend Raw Listener so it can add and remove itself to the port
	friend FDMXRawListener;

protected:
	/** Creates an output port tied to a specific config. Makes the config valid if it's invalid. */
	static FDMXInputPortSharedRef CreateFromConfig(FDMXInputPortConfig& InputPortConfig);

public:
	virtual ~FDMXInputPort();

	/** Creates a dmx input port config that corresponds to the port */
	FDMXInputPortConfig MakeInputPortConfig() const;

	/** 
	 * Updates the Port to use the config of the InputPortConfig. Makes the config valid if it's invalid. 
	 *
	 * @param InOutInputPortConfig					The config that is applied. May be changed to a valid config.
	 * @param bForceUpdateRegistrationWithProtocol	Optional: Forces the port to update its registration with the protocol (useful for runtime changes)
	 */
	void UpdateFromConfig(FDMXInputPortConfig& InOutInputPortConfig, bool bForceUpdateRegistrationWithProtocol = false);

	//~ Begin DMXPort Interface 
	virtual bool IsRegistered() const override;
	virtual const FGuid& GetPortGuid() const override;
	virtual bool CheckPriority(const int32 InPriority);
protected:
	virtual void AddRawListener(TSharedRef<FDMXRawListener> InRawListener) override;
	virtual void RemoveRawListener(TSharedRef<FDMXRawListener> InRawListenerToRemove) override;
	virtual bool Register() override;
	virtual void Unregister() override;
	//~ End DMXPort Interface

	//~ Begin FTickableGameObject interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickableInEditor() const override { return true; }
	virtual ETickableTickType GetTickableTickType() const override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject interface

public:
	/** Clears all buffers */
	void ClearBuffers();

	/** Thread-safe: Pushes a DMX Signal into the buffer */
	void InputDMXSignal(const FDMXSignalSharedRef& DMXSignal);

	/** Single Producer thread-safe: Pushes a DMX Signal into the buffer (For protocol only) */
	UE_DEPRECATED(5.0, "Input ports now support multiple producers. Use InputDMXSignal instead.")
	void SingleProducerInputDMXSignal(const FDMXSignalSharedRef& DMXSignal);

	/** Gets the last signal received in specified local universe. Returns false if no signal was received. Game-Thread only */
	bool GameThreadGetDMXSignal(int32 LocalUniverseID, FDMXSignalSharedPtr& OutDMXSignal);

	/** Gets all the last signal received. Game-Thread only */
	const TMap<int32, FDMXSignalSharedPtr>& GameThreadGetAllDMXSignals() const;

	/**  DEPRECATED 4.27. Gets the DMX signal from an extern (remote) Universe ID. */
	UE_DEPRECATED(4.27, "Use GameThreadGetDMXSignal instead. GameThreadGetDMXSignalFromRemoteUniverse only exists to support deprecated blueprint nodes.")
	bool GameThreadGetDMXSignalFromRemoteUniverse(FDMXSignalSharedPtr& OutDMXSignal, int32 RemoteUniverseID, bool bEvenIfNotLoopbackToEngine = false);

	/** Returns true if receive DMX is disabled */
	bool IsReceiveDMXEnabled() const { return bReceiveDMXEnabled; }

	/** 
	 * Injects a dmx signal into the game thread. 
	 * Note, only signals get from game thread are affected but not raw listeners. 
	 * Useful for nDisplay replication.
	 */
	void GameThreadInjectDMXSignal(const FDMXSignalSharedRef& DMXSignal);

	/** 
	 * Sets if the port should listen to its default queue. 
	 * If false, another raw listener can create the default queue by adding a raw listener and inject its data via GameThreadInjectDMXSignal
	 * Useful for nDisplay replication.
	 */
	void SetUseDefaultQueue(bool bUse);
	
private:
	/** Called to set if receive DMX should be enabled */
	void OnSetReceiveDMXEnabled(bool bEnabled);

	/** The default buffer, which is being read on tick */
	TQueue<FDMXSignalSharedPtr, EQueueMode::Mpsc> DefaultInputQueue;

	/** According to DMXProtcolSettings, true if DMX should be received */
	bool bReceiveDMXEnabled = false;

	/** If true, adds data to the default input queue and fills the UniverseToLatestSignalMap with it */
	bool bUseDefaultInputQueue = true;

	/** How to manage priority */
	EDMXPortPriorityStrategy PriorityStrategy;

	/** the Priority value, used by the PriorityStrategy */
	int32 Priority;

	/** The highest priority received */
	int32 HighestReceivedPriority = 0;

	/** The highest priority received */
	int32 LowestReceivedPriority = 0;

	/** Map of Universe Inputs with their Universes */
	TMap<int32, TSet<TSharedRef<FDMXTickedUniverseListener>>> LocalUniverseToListenerGroupMap;

	/** Map of latest Singals per extern Universe */
	TMap<int32, FDMXSignalSharedPtr> ExternUniverseToLatestSignalMap;

	/** Map of raw isteners */
	TSet<TSharedRef<FDMXRawListener>> RawListeners;

	/** Mutex access for the RawListeners array */
	FCriticalSection AccessRawListenersMutex;

	/** True while registration is pending */
	bool bPendingRegistration = false;

	/** True if the port is registered with it its protocol */
	TAtomic<bool> bRegistered = false;

	/** The unique identifier of this port, shared with the port config this was constructed from. Should not be changed after construction. */
	FGuid PortGuid;
};
