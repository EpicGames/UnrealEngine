// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolModule.h"
#include "DMXProtocolTypes.h"

#include "CoreMinimal.h"
#include "Misc/Guid.h"

#include "DMXInputPortConfig.generated.h"

struct FDMXInputPortConfig;
class FDMXPort;

struct FGuid;

/**
 * Strategy for priority system (when receiving packets)
 * 
 * Not: Not all protocols have a use for this
*/
UENUM()
enum class EDMXPortPriorityStrategy : uint8
{
	/** Always manage the packet */
	None,
	/** Manage the packet only if the priority is equal to the specified value */
	Equal,
	/** Manage the packet only if the priority is higher than the specified value */
	HigherThan,
	/** Manage the packet only if the priority is lower than the specified value */
	LowerThan,
	/** Manage the packet only if it matches the highest received priority */
	Highest,
	/** Manage the packet only if it matches the lowest received priority */
	Lowest
};

/** Data to create a new input port config with related constructor. */
struct DMXPROTOCOL_API FDMXInputPortConfigParams
{
	FDMXInputPortConfigParams() = default;
	FDMXInputPortConfigParams(const FDMXInputPortConfig& InputPortConfig);

	FString PortName;
	FName ProtocolName;
	EDMXCommunicationType CommunicationType;
	bool bAutoCompleteDeviceAddressEnabled;
	FString AutoCompleteDeviceAddress;
	FString DeviceAddress; 
	int32 LocalUniverseStart;
	int32 NumUniverses;
	int32 ExternUniverseStart;
	EDMXPortPriorityStrategy PriorityStrategy;
	int32 Priority;
};

/** 
 * Blueprint Configuration of a Port, used in DXM Settings to specify inputs and outputs.
 *
 * Property changes are handled in details customization consistently.
 */
USTRUCT(BlueprintType)
struct DMXPROTOCOL_API FDMXInputPortConfig
{
	GENERATED_BODY()

public:
	/** Default constructor, only for Default Objects */
	FDMXInputPortConfig();

	/** Constructs a config from the guid */
	explicit FDMXInputPortConfig(const FGuid& InPortGuid);

	/** Constructs a config from the guid and given initialization data */
	FDMXInputPortConfig(const FGuid& InPortGuid, const FDMXInputPortConfigParams& InitializationData);

	/** Changes members to result in a valid config */
	void MakeValid();

	/** Upgrades this config to the latest engine version */
	void Upgrade();

	const FString& GetPortName() const { return PortName; }
	const FName& GetProtocolName() const { return ProtocolName; }
	EDMXCommunicationType GetCommunicationType() const { return CommunicationType; }
	bool IsAutoCompleteDeviceAddressEnabled() const { return bAutoCompleteDeviceAddressEnabled; }
	FString GetAutoCompleteDeviceAddress() const { return AutoCompleteDeviceAddress; }
	FString GetDeviceAddress() const;
	int32 GetLocalUniverseStart() const { return LocalUniverseStart; }
	int32 GetNumUniverses() const { return NumUniverses; }
	int32 GetExternUniverseStart() const { return ExternUniverseStart; }
	const FGuid& GetPortGuid() const { return PortGuid; }
	const EDMXPortPriorityStrategy GetPortPriorityStrategy() const { return PriorityStrategy; }
	const int32 GetPriority() const { return Priority; }

#if WITH_EDITOR
	static FName GetPortNamePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, PortName); }
	static FName GetProtocolNamePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, ProtocolName); }
	static FName GetCommunicationTypePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, CommunicationType); }
	static FName GetAutoCompleteDeviceAddressEnabledPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, bAutoCompleteDeviceAddressEnabled); }
	static FName GetAutoCompleteDeviceAddressPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, AutoCompleteDeviceAddress); }
	static FName GetDeviceAddressPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, DeviceAddress); }
	static FName GetLocalUniverseStartPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, LocalUniverseStart); }
	static FName GetNumUniversesPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, NumUniverses); }
	static FName GetExternUniverseStartPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, ExternUniverseStart); }
	static FName GetIsExternUnivereStartEditablePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, bIsExternUnivereStartEditable); }
	static FName GetPriorityStrategyPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, PriorityStrategy); }
	static FName GetPriorityPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, Priority); }
	static FName GetPortGuidPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortConfig, PortGuid); }
#endif // WITH_EDITOR

protected:
	/** The name displayed wherever the port can be displayed */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	FString PortName;

	/** DMX Protocol */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Protocol Type"))
	FName ProtocolName = FDMXProtocolModule::DefaultProtocolArtNetName;

	/** The type of communication used with this port */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	EDMXCommunicationType CommunicationType = EDMXCommunicationType::InternalOnly;

	/** Enables 'Auto Complete Device Address', hidden via customization - EditConditionInlineToggle doesn't support Config */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	bool bAutoCompleteDeviceAddressEnabled = false;

	/**
	 * Searches available Network Interface Card IP Addresses and uses the first match as the 'Network Interface Card IP Address' (both in Editor and Game).
	 *
	 * Supports wildcards, examples:
	 * '192'
	 * '192.*'
	 * '192.168.?.*'.
	 *
	 * If empty or '*' the first best available IP will be selected (not recommended)
	 */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Auto Complete Network Interface Card IP Address"))
	FString AutoCompleteDeviceAddress = TEXT("192.*");

	/** The Network Interface Card's IP Adress, over which DMX is received */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Network Interface Card IP Address"))
	FString DeviceAddress = TEXT("127.0.0.1");

	/** Local Start Universe */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	int32 LocalUniverseStart = 1;

	/** Number of Universes */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Amount of Universes"))
	int32 NumUniverses = 10;

	/** 
	 * Remaps the range of local Universes (Num Universes from Local Universe Start) to a different set of Universes at the Protocol level.
	 * For example, for Art-Net this allows to map local Universe 1 to extern Universe 0. 
	 */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Protocol Universe Remap"))
	int32 ExternUniverseStart = 1;

#if WITH_EDITORONLY_DATA
	/** Edit condition for the ExternUniverseStart property */
	UPROPERTY(Config, EditDefaultsOnly, Category = "Port Config")
	bool bIsExternUnivereStartEditable = false;
#endif

	/** How to deal with the priority value */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	EDMXPortPriorityStrategy PriorityStrategy = EDMXPortPriorityStrategy::Highest;

	/** Priority value, can act as a filter or a threshold */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	int32 Priority = 0;

protected:
	
	/** 
	 * Unique identifier, shared with the port instance.
	 * Note: This needs be BlueprintReadWrite to be accessible to property type customization, but is hidden by customization.
	 */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config Guid", meta = (IgnoreForMemberInitializationTest))
	FGuid PortGuid;
};
