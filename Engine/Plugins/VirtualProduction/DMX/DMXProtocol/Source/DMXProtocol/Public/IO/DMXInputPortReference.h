// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolLog.h"
#include "DMXProtocolSettings.h"

#include "Misc/Guid.h"

#include "DMXInputPortReference.generated.h"


/** Reference of an input port */
USTRUCT(BlueprintType)
struct FDMXInputPortReference
{
	GENERATED_BODY()

	FDMXInputPortReference()
		: bEnabledFlag(true)
	{
		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		if (ProtocolSettings && ProtocolSettings->InputPortConfigs.Num() > 0)
		{
			PortGuid = ProtocolSettings->InputPortConfigs[0].GetPortGuid();
		}
	}

	FDMXInputPortReference(const FGuid& InPortGuid, bool bIsEnabledFlag)
		: PortGuid(InPortGuid)
		, bEnabledFlag(bIsEnabledFlag)
	{
		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		if (ProtocolSettings)
		{
			const FDMXInputPortConfig* ExistingInputPortConfigPtr = ProtocolSettings->InputPortConfigs.FindByPredicate([InPortGuid](const FDMXInputPortConfig& InputPortConfig)
				{
					return InputPortConfig.GetPortGuid() == InPortGuid;
				});

			UE_CLOGF(!ExistingInputPortConfigPtr, LogDMXProtocol, Warning, "Trying to construct input port refrence, but the referenced Port Guid doesn't exist.");
		}
	}

	FDMXInputPortReference(const FDMXInputPortReference& InInputPortReference, bool bIsEnabledFlag)
		: PortGuid(InInputPortReference.PortGuid)
		, bEnabledFlag(bIsEnabledFlag)
	{
		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		if (ProtocolSettings)
		{
			const FDMXInputPortConfig* ExistingInputPortConfigPtr = ProtocolSettings->InputPortConfigs.FindByPredicate([InInputPortReference](const FDMXInputPortConfig& InputPortConfig)
				{
					return InputPortConfig.GetPortGuid() == InInputPortReference.GetPortGuid();
				});

			UE_CLOGF(!ExistingInputPortConfigPtr, LogDMXProtocol, Warning, "Trying to construct input port refrence, but the referenced Port Guid doesn't exist.");
		}
	}

	/** Returns true if the port is enabled. Always true unless constructed with bIsAlwaysEnabled = false */
	bool IsEnabledFlagSet() const { return bEnabledFlag; }

	friend FArchive& operator<<(FArchive& Ar, FDMXInputPortReference& InputPortReference)
	{
		Ar << InputPortReference.PortGuid;

		return Ar;
	}

	bool operator==(const FDMXInputPortReference& Other) const
	{
		return PortGuid == Other.PortGuid;
	}

	bool operator!=(const FDMXInputPortReference& Other) const
	{
		return !(*this == Other);
	}

	FORCEINLINE_DEBUGGABLE friend uint32 GetTypeHash(const FDMXInputPortReference& PortReference)
	{
		return GetTypeHash(PortReference.PortGuid);
	}

	const FGuid& GetPortGuid() const { return PortGuid; }

	/** Returns a non const Guid of the port, required for serialization only */
	FGuid& GetMutablePortGuid() { return PortGuid; }

	static FName GetPortGuidPropertyName() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortReference, PortGuid); }
	static FName GetEnabledFlagPropertyName() { return GET_MEMBER_NAME_CHECKED(FDMXInputPortReference, bEnabledFlag); }

protected:
	/**
	 * Unique identifier shared with port config and port instance.
	 * Note: This needs be BlueprintReadWrite to be accessible to property type customization, but is hidden by customization.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX")
	FGuid PortGuid;

	/** Optional flag for port references that can be enabled or disabled */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX")
	uint32 bEnabledFlag : 1;
};
