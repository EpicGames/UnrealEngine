// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPort.h"

#include <atomic>
#include "DMXProtocolCommon.h"
#include "Containers/Queue.h" 
#include "HAL/Runnable.h"
#include "IO/DMXOutputPortConfig.h"
#include "Misc/ScopeLock.h" 
#include "Misc/SingleThreadRunnable.h"
#include "Templates/Atomic.h"
#include "Tickable.h"

class FDMXPortManager;
class FDMXSignal;
class IDMXSender;

class FEvent;


/** Helper to determine how DMX should be communicated (loopback, send) */
struct FDMXOutputPortCommunicationDeterminator
{
	FDMXOutputPortCommunicationDeterminator()
		: bLoopbackToEngine(false)
		, bReceiveEnabled(true)
		, bSendEnabled(true)
		, bHasValidSender(false)
	{}

	/** Set the variable from the port config in project settings */
	void SetLoopbackToEngine(bool bInLoopbackToEngine) { bLoopbackToEngine = bInLoopbackToEngine; }

	/** Sets if receive is enabled */
	void SetReceiveEnabled(bool bInReceiveEnabled) { bReceiveEnabled = bInReceiveEnabled; }

	/** Sets if send is enabled */
	void SetSendEnabled(bool bInSendEnabled) { bSendEnabled = bInSendEnabled; }

	/** Sets if there is a valid sender obj */
	void SetHasValidSender(bool bInHasValidSender) { bHasValidSender = bInHasValidSender; }

	/** Determinates if loopback to engine is needed (may be true even is loopback to engine is not enabled) */
	bool NeedsLoopbackToEngine() const { return bLoopbackToEngine || !bReceiveEnabled || !bSendEnabled; }

	/** Determinates if dmx needs to be sent (may be false even if send is enabled) */
	bool NeedsSendDMX() const { return bSendEnabled && bHasValidSender; }

	/** Determinates if send dmx is enabled */
	bool IsSendDMXEnabled() const { return bSendEnabled; }

	/** Returns true if loopback to engine is enabled (only true if loopback to engine is set enabled) */
	bool IsLoopbackToEngineEnabled() const { return bLoopbackToEngine; }

private:
	TAtomic<bool> bLoopbackToEngine;
	TAtomic<bool> bReceiveEnabled;
	TAtomic<bool> bSendEnabled;
	TAtomic<bool> bHasValidSender;
};


/** Structs that holds the fragmented values to be sent along with a timestamp when to send it */
struct DMXPROTOCOL_API FDMXSignalFragment
	: TSharedFromThis<FDMXSignalFragment, ESPMode::ThreadSafe>
{
	FDMXSignalFragment() = delete;

	FDMXSignalFragment(int32 InExternUniverseID, const TMap<int32, uint8>& InChannelToValueMap, double InSendTime)
		: ExternUniverseID(InExternUniverseID)
		, ChannelToValueMap(InChannelToValueMap)
		, SendTime(InSendTime)
	{}

	/** The universe the fragment needs to be written to */
	int32 ExternUniverseID;

	/** The map of channels and values to send */
	TMap<int32, uint8> ChannelToValueMap;

	/** The time when the Fragment needs be sent */
	double SendTime;
};


/**
 * Higher level abstraction of a DMX input hiding networking specific and protocol specific complexity. Ment to be constructed by DMXPortManger only.
 *
 * Use SendDMX method to send DMX.
 *
 * To loopback outputs, refer to DMXRawListener.
 */
class DMXPROTOCOL_API FDMXOutputPort
	: public FDMXPort
	, public FRunnable
	, public FSingleThreadRunnable
{
	// Friend DMXPortManager so it can create instances and unregister void instances
	friend FDMXPortManager;

	// Friend Raw Listener so it can add and remove itself to the port
	friend FDMXRawListener;

protected:
	/** Creates an output port tied to a specific config. Hidden on purpose, use FDMXPortManager to create instances */
	static FDMXOutputPortSharedRef CreateFromConfig(FDMXOutputPortConfig& OutputPortConfig);

public:
	virtual ~FDMXOutputPort();

	/** Creates a dmx output port config that corresponds to the port */
	FDMXOutputPortConfig MakeOutputPortConfig() const;

	/**
	 * Updates the Port to use the config of the OutputPortConfig. Makes the config valid if it's invalid.
	 *
	 * @param InOutInputPortConfig					The config that is applied. May be changed to a valid config.
	 * @param bForceUpdateRegistrationWithProtocol	Optional: Forces the port to update its registration with the protocol (useful for runtime changes)
	 */
	void UpdateFromConfig(FDMXOutputPortConfig& OutputPortConfig, bool bForceUpdateRegistrationWithProtocol = false);

	/** Sends DMX over the port */
	void SendDMX(int32 LocalUniverseID, const TMap<int32, uint8>& ChannelToValueMap);

	/** DEPRECATED 4.27. Sends DMX over the port with an extern (remote) Universe ID. Soly here to support legacy functions that would send to an extern universe  */
	UE_DEPRECATED(4.27, "Use SenDMX instead. SendDMXToRemoteUniverse only exists to support deprecated blueprint nodes.")
	void SendDMXToRemoteUniverse(const TMap<int32, uint8>& ChannelToValueMap, int32 RemoteUniverse);

	/** Clears all buffers */
	void ClearBuffers();

	/** 
	 * Game-Thread only: Gets the last signal received in specified local universe. 
	 * 
	 * @param LocalUniverseID				The local universe that should be retrieved
	 * @param OutDMXSignal					The signal that is set if the opperation succeeds.
	 * @param bEvenIfNotLoopbackToEngine	Defaults to false. If true, succeeds even if the signal should not be looped back to engine (useful for monitoring).
	 * @return								True if the OutDMXSignal was set.
	 */
	bool GameThreadGetDMXSignal(int32 LocalUniverseID, FDMXSignalSharedPtr& OutDMXSignal, bool bEvenIfNotLoopbackToEngine);

	/** Returns the Destination Addresses */
	TArray<FString> GetDestinationAddresses() const;

	/**  DEPRECATED 4.27. Gets the DMX signal from an extern (remote) Universe ID. */
	UE_DEPRECATED(4.27, "Use GameThreadGetDMXSignal instead. GameThreadGetDMXSignalFromRemoteUniverse only exists to support deprecated blueprint nodes.")
	bool GameThreadGetDMXSignalFromRemoteUniverse(FDMXSignalSharedPtr& OutDMXSignal, int32 RemoteUniverseID, bool bEvenIfNotLoopbackToEngine);

	/** Returns the multicast Time-to-Live (TTL), only used when the port is set to multicast */
	uint8 GetTimeToLive() const { return TimeToLive; }

	/** Returns the output port's delay in seconds */
	double GetDelaySeconds() const { return DelaySeconds; }

public:
	//~ Begin DMXPort Interface 
	virtual bool IsRegistered() const override;
	virtual const FGuid& GetPortGuid() const override;

protected:
	virtual void AddRawListener(TSharedRef<FDMXRawListener> InRawListener) override;
	virtual void RemoveRawListener(TSharedRef<FDMXRawListener> InRawListenerToRemove) override;
	virtual bool Register() override;
	virtual void Unregister() override;
	//~ End DMXPort Interface

	/** Called when the DMX send rate changed */
	void OnSendingRefreshRateChanged();

	/** Called to set if DMX should be enabled */
	void OnSetSendDMXEnabled(bool bEnabled);

	/** Called to set if DMX should be enabled */
	void OnSetReceiveDMXEnabled(bool bEnabled);

	//~ Begin FRunnable implementation
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;
	//~ End FRunnable implementation

	//~ Begin FSingleThreadRunnable implementation
	virtual void Tick() override;
	virtual class FSingleThreadRunnable* GetSingleThreadInterface() override;
	//~ End FSingleThreadRunnable implementation

	/** Updates the thread, sending DMX */
	void ProcessSendDMX();

private:
	/** The DMX senders in use */
	TArray<TSharedPtr<IDMXSender>> DMXSenderArray;

	/** Buffer of the signals that are to be sent in the next frame */
	TQueue<TSharedPtr<FDMXSignalFragment, ESPMode::ThreadSafe>> SignalFragments;

	/** Map that holds the latest Signal per Universe on the Game Thread */
	TMap<int32, FDMXSignalSharedPtr> ExternUniverseToLatestSignalMap_GameThread;

	/** Map that holds the latest Signal per Universe on the Port Thread */
	TMap<int32, FDMXSignalSharedPtr> ExternUniverseToLatestSignalMap_PortThread;

	/** The Destination Address to send to, can be irrelevant, e.g. for art-net broadcast */
	TArray<FDMXOutputPortDestinationAddress> DestinationAddresses;

	/** Helper to determine how dmx should be communicated (loopback, send) */
	FDMXOutputPortCommunicationDeterminator CommunicationDeterminator;

	/** Mutex access to members required to send DMX  */
	mutable FCriticalSection SendDMXMutex;

	/** Multicast Time-to-Live (TTL), only used when the port is set to multicast */
	uint8 TimeToLive = 1;

	/** Priority on which packets are being sent */
	int32 Priority = 0;

	/** Map of raw Inputs */
	TSet<TSharedRef<FDMXRawListener>> RawListeners;

	/** True while registration is pending */
	bool bPendingRegistration = false;

	/** True if the port is registered with its protocol */
	TAtomic<bool> bRegistered = false;

	/** The unique identifier of this port, shared with the port config this was constructed from. Should not be changed after construction. */
	FGuid PortGuid;

	/** Current send rate, set from project settings (see OnSendingRefreshRateChanged) */
	TAtomic<uint32> SendRate = 44.f;

	/** Delay to apply on packets being sent */
	double DelaySeconds = 0.0;

	/** The frame rate of the delay */
	FFrameRate DelayFrameRate;

	/** Send DMX sync event */
	FEvent* SendDMXEvent = nullptr;

	/** Holds this thread object */
	FRunnableThread* Thread = nullptr;

	/** Flag indicating that the thread is stopping. */
	TAtomic<bool> bStopping;
};
