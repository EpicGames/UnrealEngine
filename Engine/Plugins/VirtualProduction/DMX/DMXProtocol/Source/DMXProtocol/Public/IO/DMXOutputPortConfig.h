// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolModule.h"
#include "DMXProtocolTypes.h"

#include "CoreMinimal.h"
#include "Misc/Guid.h"

#include "DMXOutputPortConfig.generated.h"

struct FDMXOutputPortConfig;
class FDMXPort;

struct FGuid;


/** The IP address outbound DMX is sent to */
USTRUCT(BlueprintType)
struct DMXPROTOCOL_API FDMXOutputPortDestinationAddress
{
	GENERATED_BODY()

	FDMXOutputPortDestinationAddress() = default;

	FDMXOutputPortDestinationAddress(const FString& DestinationAddress)
		: DestinationAddressString(DestinationAddress)
	{}

	/** The IP address outbound DMX is sent to */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Destination IP Address"))
	FString DestinationAddressString;

	bool operator==(const FDMXOutputPortDestinationAddress& Other) const { return DestinationAddressString == Other.DestinationAddressString; }
	bool operator!=(const FDMXOutputPortDestinationAddress& Other) const { return !operator==(Other); }
	friend uint32 GetTypeHash(const FDMXOutputPortDestinationAddress& Key) { return GetTypeHash(Key.DestinationAddressString); }
};

/** Data to create a new output port config with related constructor */
struct DMXPROTOCOL_API FDMXOutputPortConfigParams
{
	FDMXOutputPortConfigParams() = default;
	FDMXOutputPortConfigParams(const FDMXOutputPortConfig& OutputPortConfig);

	FString PortName;
	FName ProtocolName;
	EDMXCommunicationType CommunicationType;
	uint8 TimeToLive;
	bool bAutoCompleteDeviceAddressEnabled;
	FString AutoCompleteDeviceAddress;
	FString DeviceAddress;
	TArray<FDMXOutputPortDestinationAddress> DestinationAddresses;
	bool bLoopbackToEngine;
	int32 LocalUniverseStart;
	int32 NumUniverses;
	int32 ExternUniverseStart;
	int32 Priority;
	double Delay;
	FFrameRate DelayFrameRate;

	// DEPRECATED 5.0. Instead please use the DestinationAddresses array.
	FString DestinationAddress_DEPRECATED;
};

/** 
 * Blueprint Configuration of a Port, used in DXM Settings to specify inputs and outputs.
 *
 * Property changes are handled in details customization consistently.
 */
USTRUCT(BlueprintType)
struct DMXPROTOCOL_API FDMXOutputPortConfig
{
	GENERATED_BODY()

public:
	/** Default constructor, only for Default Objects */
	FDMXOutputPortConfig();

	/** Constructs a config from the guid */
	explicit FDMXOutputPortConfig(const FGuid & InPortGuid);

	/** Constructs a config from the guid and given initialization data */
	FDMXOutputPortConfig(const FGuid & InPortGuid, const FDMXOutputPortConfigParams& InitializationData);

	/** Changes members to result in a valid config */
	void MakeValid();

	/** Upgrades this config to the latest engine version */
	void Upgrade();

	const FString& GetPortName() const { return PortName; }
	const FName& GetProtocolName() const { return ProtocolName; }
	EDMXCommunicationType GetCommunicationType() const { return CommunicationType; }
	uint8 GetTimeToLive() const { return TimeToLive; }
	bool IsAutoCompleteDeviceAddressEnabled() const { return bAutoCompleteDeviceAddressEnabled; }
	FString GetAutoCompleteDeviceAddress() const { return AutoCompleteDeviceAddress; }
	FString GetDeviceAddress() const;
	const TArray<FDMXOutputPortDestinationAddress>& GetDestinationAddresses() const { return DestinationAddresses; }
	bool NeedsLoopbackToEngine() const { return bLoopbackToEngine; }
	int32 GetLocalUniverseStart() const { return LocalUniverseStart; }
	int32 GetNumUniverses() const { return NumUniverses; }
	int32 GetExternUniverseStart() const { return ExternUniverseStart; }
	int32 GetPriority() const { return Priority; }
	const FGuid& GetPortGuid() const { return PortGuid; }
	double GetDelay() const { return Delay; }
	const FFrameRate& GetDelayFrameRate() const { return DelayFrameRate; }

	UE_DEPRECATED(5.0, "Output Ports now support many destination addresses. Use FDMXOutputPortConfig::GetDestinationAddresses instead. ")
	const FString& GetDestinationAddress() const { return DestinationAddress; }

#if WITH_EDITOR
	static FName GetPortNamePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, PortName); }
	static FName GetProtocolNamePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, ProtocolName); }
	static FName GetCommunicationTypePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, CommunicationType); }
	static FName GetAutoCompleteDeviceAddressEnabledPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, bAutoCompleteDeviceAddressEnabled); }
	static FName GetAutoCompleteDeviceAddressPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, AutoCompleteDeviceAddress); }
	static FName GetDeviceAddressPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, DeviceAddress); }
	static FName GetDestinationAddressesPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, DestinationAddresses); }
	static FName GetLocalUniverseStartPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, LocalUniverseStart); }
	static FName GetNumUniversesPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, NumUniverses); }
	static FName GetExternUniverseStartPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, ExternUniverseStart); }
	static FName GetIsExternUnivereStartEditablePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, bIsExternUnivereStartEditable); }
	static FName GetPriorityPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, Priority); }
	static FName GetDelayPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, Delay); }
	static FName GetDelayFrameRatePropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, DelayFrameRate); }
	static FName GetPortGuidPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, PortGuid); }

	UE_DEPRECATED(5.0, "Output Ports now support many destination addresses. Use FDMXOutputPortConfig::GetDestinationAddressesPropertyNameChecked instead. ")
	static FName GetDestinationAddressPropertyNameChecked() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortConfig, DestinationAddress); }
#endif // WITH_EDITOR

protected:
	/** The name displayed wherever the port can be displayed */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	FString PortName;

	/** DMX Protocol */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Protocol Type"))
	FName ProtocolName = FDMXProtocolModule::DefaultProtocolArtNetName;

	/** The type of communication used with this port */
	UPROPERTY(Config, BlueprintReadWrite, EditDefaultsOnly, Category = "Port Config")
	EDMXCommunicationType CommunicationType = EDMXCommunicationType::InternalOnly;

	/** Sets how many hops multicast DMX packets can take before being discarded. In common network setups this should be set to 1. */
	UPROPERTY()
	uint8 TimeToLive = 1;

	/** Enables 'Auto Complete Device Address', hidden via customization - EditConditionInlineToggle doesn't support Config */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	bool bAutoCompleteDeviceAddressEnabled = false;

	/**
	 * Searches available Network Interface Card IP Addresses and uses the first match as the 'Network Interface Card IP Address' (both in Editor and Game).
	 *
	 * Supports wildcards, examples:
	 * '192'
	 * '192.*'
	 * '192.168.?.*'..
	 *
	 * If empty or '*' the first best available IP will be selected (not recommended)
	 */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Auto Complete Network Interface Card IP Address"))
	FString AutoCompleteDeviceAddress = TEXT("192.*");

	/** The IP address of the network interface card over which outbound DMX is sent */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Network Interface Card IP Address"))
	FString DeviceAddress = TEXT("127.0.0.1");

	/** For Unicast, the IP address outbound DMX is sent to */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "DestinationAddress is deprecated. Please use DestinationAddresses instead."))
	FString DestinationAddress = TEXT("None");

	/** For Unicast, the IP addresses outbound DMX is sent to */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Destination IP Address"))
	TArray<FDMXOutputPortDestinationAddress> DestinationAddresses;

	/** If true, the signals output from this port are input into to the engine. Note, signals input into the engine this way will not be visible in Monitors when monitoring Inputs. */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Input into Engine"))
	bool bLoopbackToEngine = true;

	/** Local Start Universe */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	int32 LocalUniverseStart = 1;

	/** Number of Universes */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Amount of Universes"))
	int32 NumUniverses = 10;

	/**
	 * Remaps the range of local Universes (Num Universes from Local Universe Start) to a different set of Universes at the Protocol level.
	 * For example, for Art-Net this allows to map local Universe 1 to extern Universe 0.
	 */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config", Meta = (DisplayName = "Protocol Universe Remap"))
	int32 ExternUniverseStart = 1;

#if WITH_EDITORONLY_DATA
	/** Edit condition for the ExternUniverseStart property */
	UPROPERTY(Config, EditDefaultsOnly, Category = "Port Config")
	bool bIsExternUnivereStartEditable = false;
#endif

	/** Priority on which packets are being sent */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	int32 Priority = 100;

	/** The amout by which sending of packets is delayed */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	double Delay = 0.0;

	/** Framerate of the delay */
	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config")
	FFrameRate DelayFrameRate;

protected:

	/** 
	 * Unique identifier, shared with the port instance.
	 * Note: This needs be BlueprintReadWrite to be accessible to property type customization, but is hidden by customization.
	 */

	UPROPERTY(Config, BlueprintReadOnly, EditDefaultsOnly, Category = "Port Config Guid", meta = (IgnoreForMemberInitializationTest))
	FGuid PortGuid;
};
