// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolLog.h"
#include "DMXProtocolSettings.h"

#include "Misc/Guid.h"

#include "DMXOutputPortReference.generated.h"


/** Reference of an input port */
USTRUCT(BlueprintType)
struct FDMXOutputPortReference
{
	GENERATED_BODY()

	FDMXOutputPortReference()
		: bEnabledFlag(true)
	{
		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		if (ProtocolSettings && ProtocolSettings->OutputPortConfigs.Num() > 0)
		{
			PortGuid = ProtocolSettings->OutputPortConfigs[0].GetPortGuid();
		}
	}

	FDMXOutputPortReference(const FGuid& InPortGuid, bool bIsEnabledFlag)
		: PortGuid(InPortGuid)
		, bEnabledFlag(bIsEnabledFlag)
	{
		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		if (ProtocolSettings)
		{
			const FDMXOutputPortConfig* ExistingOutputPortConfigPtr = ProtocolSettings->OutputPortConfigs.FindByPredicate([InPortGuid](const FDMXOutputPortConfig& OutputPortConfig)
				{
					return OutputPortConfig.GetPortGuid() == InPortGuid;
				});

			UE_CLOGF(!ExistingOutputPortConfigPtr, LogDMXProtocol, Warning, "Trying to construct output port refrence, but the referenced Port Guid doesn't exist.");
		}
	}

	FDMXOutputPortReference(const FDMXOutputPortReference& InOutputPortReference, bool bIsEnabledFlag)
		: PortGuid(InOutputPortReference.PortGuid)
		, bEnabledFlag(bIsEnabledFlag)
	{
		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		if (ProtocolSettings)
		{
			const FDMXOutputPortConfig* ExistingOutputPortConfigPtr = ProtocolSettings->OutputPortConfigs.FindByPredicate([InOutputPortReference](const FDMXOutputPortConfig& OutputPortConfig)
				{
					return OutputPortConfig.GetPortGuid() == InOutputPortReference.GetPortGuid();
				});

			UE_CLOGF(!ExistingOutputPortConfigPtr, LogDMXProtocol, Warning, "Trying to construct output port refrence, but the referenced Port Guid doesn't exist.");
		}
	}

	/** Returns true if the port is enabled. Always true unless constructed with bIsAlwaysEnabled = false */
	bool IsEnabledFlagSet() const { return bEnabledFlag; }

	friend FArchive& operator<<(FArchive& Ar, FDMXOutputPortReference& OutputPortReference)
	{
		Ar << OutputPortReference.PortGuid;

		return Ar;
	}

	bool operator==(const FDMXOutputPortReference& Other) const
	{
		return PortGuid == Other.PortGuid;
	}

	bool operator!=(const FDMXOutputPortReference& Other) const
	{
		return !(*this == Other);
	}

	FORCEINLINE_DEBUGGABLE friend uint32 GetTypeHash(const FDMXOutputPortReference& PortReference)
	{
		return GetTypeHash(PortReference.PortGuid);
	}

	const FGuid& GetPortGuid() const { return PortGuid; }

	static FName GetPortGuidPropertyName() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortReference, PortGuid); }
	static FName GetEnabledFlagPropertyName() { return GET_MEMBER_NAME_CHECKED(FDMXOutputPortReference, bEnabledFlag); }

protected:
	/**
	 * Unique identifier shared with port config and port instance.
	 * Note: This needs be BlueprintReadWrite to be accessible to property type customization, but is hidden by customization.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX")
	FGuid PortGuid;

	/** Optional flag for port references that can be enabled or disabled */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX")
	bool bEnabledFlag;
};
