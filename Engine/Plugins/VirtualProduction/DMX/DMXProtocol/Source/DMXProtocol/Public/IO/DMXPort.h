// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

enum class EDMXCommunicationType : uint8;
class FDMXRawListener;
class FDMXTickedUniverseListener;

class FInternetAddr;


/** 
 * Base class for a higher level abstraction of a DMX input or output. 
 * Higher level abstraction of a DMX input hiding networking specific and protocol specific complexity.
 */
class DMXPROTOCOL_API FDMXPort
	: public TSharedFromThis<FDMXPort, ESPMode::ThreadSafe>
{
	// Friend Inputs so they can add and remove themselves to the port
	friend FDMXRawListener;

protected:
	/** Protected default constructor, child classes need to take care of member initialization */
	FDMXPort()
	{}

	///////////////////////
	// ~Begin DMXPort Interface declaration
public:
	/** Returns true if the port is successfully registered with its protocol */
	virtual bool IsRegistered() const = 0;

	/** Returns the Guid of the Port */
	virtual const FGuid& GetPortGuid() const = 0;
	
protected:	
	/**
	 * Adds a Raw Listener that receives all raw signals received on this port.
	 * Only useful for objects that want to process all data, and not just data on tick (e.g. DMX Take Recorder).
	 * Should not be used directly, instead see DMXRawListener constructor.
	 */
	virtual void AddRawListener(TSharedRef<FDMXRawListener> InRawListener) = 0;

	/** Removes the Raw Listener from the port. Should not be used directly, instead see DMXRawListener. */
	virtual void RemoveRawListener(TSharedRef<FDMXRawListener> InRawListener) = 0;

	/** Registers the port with its protocol. Returns true if successfully registered */
	virtual bool Register() = 0;

	/** Unregisteres the port if it was registered with its protocol */
	virtual void Unregister() = 0;
	
	// ~End DMXPort Interface declaration
	///////////////////////

public:
	virtual ~FDMXPort()
	{};

	/** Returns true if the Intern Universe is in this Port's Universe range */
	bool IsLocalUniverseInPortRange(int32 Universe) const;

	/** Returns true if the Extern Universe is in this Port's Universe range */
	bool IsExternUniverseInPortRange(int32 Universe) const;

	/** Returns the offset of the extern universe. LocalUniverse == ExternUniverse - ExternUniverseOffset */
	int32 GetExternUniverseOffset() const;

	/** Converts an extern Universe ID to a local Universe ID */
	int32 ConvertExternToLocalUniverseID(int32 ExternUniverseID) const;

	/** Converts a local Universe ID to an extern Universe ID */
	int32 ConvertLocalToExternUniverseID(int32 LocalUniverseID) const;

	const FString& GetPortName() const { return PortName; }

	const IDMXProtocolPtr& GetProtocol() const { return Protocol; }

	bool IsAutoCompleteDeviceAddressEnabled() const { return bAutoCompleteDeviceAddressEnabled; }

	const FString& GetAutoCompleteDeviceAddress() const { return AutoCompleteDeviceAddress; }

	const FString& GetDeviceAddress() const { return DeviceAddress; }

	EDMXCommunicationType GetCommunicationType() const { return CommunicationType; }

	int32 GetLocalUniverseStart() const { return LocalUniverseStart; }

	int32 GetLocalUniverseEnd() const { return LocalUniverseStart + NumUniverses - 1; }

	int32 GetExternUniverseStart() const { return ExternUniverseStart; }

	 int32 GetExternUniverseEnd() const { return ExternUniverseStart + NumUniverses - 1; }

	/** Broadcast when the port is updated */
	FSimpleMulticastDelegate OnPortUpdated;

protected:
	/** Tests whether the port is valid */
	bool IsValidPortSlow() const;

	////////////////////////////////////////////////////////////////
	// Variables that need be initialized from derived classes
protected:
	/** The name displayed wherever the port can be displayed */
	FString PortName;

	/** The protocol of this port */
	IDMXProtocolPtr Protocol;

	/** The communication type of this port */
	EDMXCommunicationType CommunicationType;

	/** If true, instead of using the Device Address, the port will auto-complete the IP from the available Network Interface Card Addresses */
	bool bAutoCompleteDeviceAddressEnabled = false;

	/** The auto-complete Device Address, supports wildcards */
	FString AutoCompleteDeviceAddress;

	/** The address of the device that handles communication, e.g. the network interface for art-net */
	FString DeviceAddress;

	/** The Local Start Universe */
	int32 LocalUniverseStart;

	/** Number of Universes */
	int32 NumUniverses;

	/**
	 * The start address this being transposed to.
	 * E.g. if LocalUniverseStart is 1 and this is 100, Local Universe 1 is sent as Universe 100.
	 */
	int32 ExternUniverseStart;
};
