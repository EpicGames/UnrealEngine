// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"

#include "Templates/SharedPointer.h"

struct FDMXInputPortConfig;
struct FDMXOutputPortConfig;
class FDMXPort;
class FDMXInputPort;
class FDMXOutputPort;


DECLARE_MULTICAST_DELEGATE(FDMXOnPortsChangedDelegate);

/** 
 * Manager for all DMX ports. Exposes available input and output ports anywhere.
 *
 * A) Overview of the IO System
 * ============================
 *
 * 1. Definition of Ports in Project Settings:
 * -------------------------------------------
 * DMX Protocol Settings (the DMX Project Settings) holds two arrays of DMX Input Port Configs resp. DMX Output Port Configs. This is where ports are defined.
 *
 * Port Manager automatically creates DMXInputPortS and DMXOutputPortS that match these settings. Generally this is self-contained and does not need any user code.
 *
 * Note: Applications that want to offer dynamic ports should specify a fixed number of ports in project settings, e.g. 8, 16 ports and work with these at runtime.
 *		 (While it is possible to mutate the protocol settings at runtime and create ports from that via FDMXPortManager.Get()::UpdateFromProtocolSettings(), 
 *		 ports created alike are not available in blueprints as port references, hence this approach is disfavored in most situations.)
 * 
 * 
 * 2. Acquire a DMXInputPort or DMXOutputPort:
 * -------------------------------------------
 * Get all input ports or all output ports available via the Port Manager's GetInputPorts and GetOutputPorts methods.
 * Alternatively use the Editor-Only SDMXPortSelector widget to select a port from available ports.
 * 
 *
 * 3. Receive DMX in an object:
 * ----------------------------------
 * a) Latest (frame time relevant) data on the Game-Thread:
 * 
 * - Call the port's GameThreadGetDMXSignal method to get a DMX Signal of a local universe.
 * 
 * b) All data on any thread:
 * 
 * - Create an instance of DMXRawListener. Use its constructor to specify which port it should use.
 * - Use DequeueSignal to receive DMX.
 *
 * Note: This applies for both input and output ports. This is to provide loopback functionality for outputs.
 *		 Generally you want to listen to input and output ports, not just the inputs. 
 *
 * Note: GameThreadGetDMXSignal is the right method to use for almost any use-case.
 *		 DMXRawListener is only useful where the latest data isn't sufficient, e.g. to record all incoming data in Sequencer.
 *		 DMXRawListener is thread-safe, but may stall the engine when used in the game-thread due to the possibly infinite work load it leaves to the user.
 *
 * 4. Send DMX from your object:
 * -------------------------------
 * Use the DMXOutputPort's SendDMX method to send DMX.
 *
 */
class DMXPROTOCOL_API FDMXPortManager
{
public:
	/** Broadcast when port arrays or data changed */
	FDMXOnPortsChangedDelegate OnPortsChanged;

	static FDMXPortManager& Get();

	/** Returns the current input ports */
	const TArray<FDMXInputPortSharedRef>& GetInputPorts() const { return InputPorts; }

	/** Returns the current output ports */
	const TArray<FDMXOutputPortSharedRef>& GetOutputPorts() const { return OutputPorts; }

	/** Clears all buffered data of Input Ports, Output Ports and Raw Listeners. Note, this clears the data, it does not zero it out.*/
	void ClearBuffers();

	/** Returns a delegate broadcast when port buffers were cleared */
	FSimpleMulticastDelegate& GetOnBuffersCleared() { return OnBuffersClearedDelegate; }

	/** Gets the input port that corresponds to the input port config. Checks the config is in the DMXProtocolSetting's InputPortConfigs array. */
	FDMXInputPortSharedRef GetInputPortFromConfigChecked(const FDMXInputPortConfig& InputPortConfig);

	/** Gets the output port that corresponds to the input port config. Checks the config is in the DMXProtocolSetting's OutputPortConfigs array. */
	FDMXOutputPortSharedRef GetOutputPortFromConfigChecked(const FDMXOutputPortConfig& OutputPortConfig);

	/** Returns the port matching the port guid. Returns nullptr if the port doesn't exist. */
	FDMXPortSharedPtr FindPortByGuid(const FGuid& PortGuid) const;

	/** Returns the port matching the port guid, checked version. */
	FDMXPortSharedRef FindPortByGuidChecked(const FGuid& PortGuid) const; 

	/** Returns the input port matching the port guid. Returns nullptr if the port doesn't exist. */
	FDMXInputPortSharedPtr FindInputPortByGuid(const FGuid& PortGuid) const;

	/** Returns the input port matching the port guid, checkedversion.*/
	FDMXInputPortSharedRef FindInputPortByGuidChecked(const FGuid& PortGuid) const;

	/** Returns the output port matching the port guid. Returns nullptr if the port doesn't exist. */
	FDMXOutputPortSharedPtr FindOutputPortByGuid(const FGuid& PortGuid) const;

	/** Returns the output port matching the port guid, checked version. */
	FDMXOutputPortSharedRef FindOutputPortByGuidChecked(const FGuid& PortGuid) const;

	/** 
	 * Updates ports from protocol settings. Useful when project settings were modified in some way. 
	 * If optional bForceUpdateRegistrationWithProtocol is set to true, ports will reregister with the protocol.
	*/
	void UpdateFromProtocolSettings(bool bForceUpdateRegistrationWithProtocol = false);

	/** 
	 * Thread-safe: Suspends the protocols irrevocably, and with it all sending and receiving of DMX via the protocols. 
	 * Useful when UE4 self wants to inject data into ports, rather than communicating with the outside world, e.g. for nDisplay replication.
	 * 
	 * Note: When protocols are suspended, changes to project settings are no longer reflected and UpdateFromProtocolSettings has no effect.
	 */
	void SuspendProtocols();

	/** Returns true if protocols are suspended */
	bool AreProtocolsSuspended() { return bProtocolsSuspended; }

private:
	/** Gets the input port that corresponds to the input port config or creates a new one. Note, the config needs be in the DMXProtocolSetting input ports array. */
	FDMXInputPortSharedRef GetOrCreateInputPortFromConfig(FDMXInputPortConfig& InputPortConfig);

	/** Removes the input port. Checks that it exists. */
	void RemoveInputPortChecked(const FGuid& PortGuid);

	/** Gets the output port that corresponds to the input port config or creates a new one. Note, the config needs be in the DMXProtocolSetting input ports array. */
	FDMXOutputPortSharedRef GetOrCreateOutputPortFromConfig(FDMXOutputPortConfig& OutputPortConfig);

	/** Removes the output port. Checks that it exists. */
	void RemoveOutputPortChecked(const FGuid& PortGuid);

	/** Array of input ports */
	TArray<FDMXInputPortSharedRef> InputPorts;

	/** Array of output ports */
	TArray<FDMXOutputPortSharedRef> OutputPorts;

	/** Array of Port Guids added from protocol settings */
	TArray<FGuid> PortGuidsFromProtocolSettings;

	/** True once Input and Output Ports are available */
	bool bIOsAvailable = false;

	/** True when protocols are suspended */
	bool bProtocolsSuspended = false;

	/** Delegate raised when buffers were cleared */
	FSimpleMulticastDelegate OnBuffersClearedDelegate;

	////////////////////////////////////////////////////////////
	// Initialization 
public:
	FDMXPortManager() = default;
	virtual ~FDMXPortManager();

	// Non-copyable
	FDMXPortManager(FDMXPortManager const&) = delete;
	void operator=(FDMXPortManager const& x) = delete;

private:
	static TUniquePtr<FDMXPortManager> CurrentManager;

public:
	/** Initializes the manager */
	static void StartupManager();

	/** Destroys the manager. */
	static void ShutdownManager();
};
