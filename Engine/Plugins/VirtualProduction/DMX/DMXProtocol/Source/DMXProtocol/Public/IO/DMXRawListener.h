// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"

#include <atomic>
#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"

class FDMXSignal;
class FDMXPort;


/**
 * Listen to all DMX data of a port.  
 * 
 * Thread-safe single producer single consumer:
 * Single producer always is the DMX port. This is handled by the port internally. Do not use EnqueueSignal somewhere else to retain thread-safety.
 * Single consumer is the user thread.
 *
 * Needs frequent dequeuing.
 * Stop needs to be called before the Input is released.
 */
class DMXPROTOCOL_API FDMXRawListener
	: public TSharedFromThis<FDMXRawListener>
{
public:
	FDMXRawListener() = delete;

	/** Constructor */
	FDMXRawListener(TSharedRef<FDMXPort, ESPMode::ThreadSafe> InOwnerPort);

	/** Destructor */
	virtual ~FDMXRawListener();

	/** Starts receiving the Input */
	void Start();

	/** Stops receiving the Input */
	void Stop();

	/** 
	 * Thread safe single producer. Enqueues a signal. Should not be called besides from the owner port.
	 *
	 * @Param Producer				The producer object, tested in debug builds to be the owner port.
	 * @Param Signal				The signal to enqueue
	 */
	virtual void EnqueueSignal(void* Producer, const FDMXSignalSharedRef& Signal);

	/** 
	 * Thread safe single consumer. Tries to dequeues a signal from the raw listener. 
	 *
	 * @param Consumer				The consumer object, tested to be the same single consumer in debug builds.
	 * @param OutSignal				The dequeued signal. If the function returns true, the outsignal is a valid shared pointer.
	 * @param OutLocalUniverseID	The local universe of the signal.
	 *
	 * @return						True if a signal was dequeued.
	 */
	bool DequeueSignal(void* Consumer, FDMXSignalSharedPtr& OutSignal, int32& OutLocalUniverseID);

	/** Thread-safe. Clears the raw buffer */
	void ClearBuffer();

private:
	/** Called when the port changed */
	void OnPortUpdated();

	/** The actual buffer */
	TQueue<FDMXSignalSharedPtr, EQueueMode::Spsc> RawBuffer;

	/** The port that owns this Listener */
	TWeakPtr<FDMXPort, ESPMode::ThreadSafe> OwnerPort;

	/** The extern universe offset of the port */
	TAtomic<int32> ExternUniverseOffset;

	/** True when stopped */
	bool bStopped;

#if UE_BUILD_DEBUG
	/** The producer object, should always be the owner port */
	void* ProducerObj;

	/** The consumer of the queue, should never change once set*/
	void* ConsumerObj;
#endif // UE_BUILD_DEBUG
};
