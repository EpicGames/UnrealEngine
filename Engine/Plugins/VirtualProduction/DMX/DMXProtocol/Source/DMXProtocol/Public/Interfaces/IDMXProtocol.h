// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"
#include "Stats/Stats.h"
#include "Modules/ModuleManager.h"
#include "DMXProtocolModule.h"

#include "Interfaces/IDMXProtocolBase.h"


enum class EDMXCommunicationType : uint8;
class FDMXSignal;
class FDMXInputPort;
class FDMXOutputPort;
class IDMXSender;


/**  
 * Serves as a higher level abstraction of a specific protocol such as Art-Net or sACN with the purpose to hide its complexity. 
 * 
 * By that the commonly expected responsability of the implementation is to:
 * - Provide a way to send and receive protocol specific DMX from and to the system.
 * - Push received DXM as generic DMXSignals to the registered DMXInputPorts.
 * - Pull generic DMXSignals from the registered DMXOutputPorts and send it as Protocol specific DMX. 
 *
 * For an overview for DMX and protocol developers, see DMXPortManager (in IO folder)
 * For an exemplary implementation, see DMXProtocolArtNet (in ProtocolArtNetModule)
 *
 * DMX Protocol Life-Cycle 
 * =======================
 *
 * Startup:
 *
 *  1. All DMX Modules should be set to load in the Predefault loading phase and bind to FDMXProtocolModule::GetOnRequestProtocolRegistration().
 * 
 *  2. At the end of the Pre-Default loading phase DMXProtocol Broadcasts GetOnRequestProtocolRegistration.
 *     Protocol should now provide their Name and DMXProtocolFactory.
 *     All protocols registered can be selected in Project Settings -> Plugins -> DMX -> Communication settings
 * 
 *  3. When a Input Port of a specific Protocol is created in Project Settings:
 *		a) The input Port calls IDMXProtocol::RegisterInputPort. 
 *         If the Protocol cannot handle the Input Port, e.g. because the network interface is not reachable, the protocol needs to return false.
 *		b) The Protocol should pass received DMX by calling DMXInputPort::SingleProducerInputDMXSignal to the Input Ports. 
 *		   Note: The Input Ports do the required filtering for relevant universes.
 *
 *	   When an Output port of a specific Protocol is created in Project Settings:
 *		a) The protocol needs to return an object that implements IDMXSender for each registered OutputPort.
 *         If the Protocol cannot handle the Output Port, e.g. because the network interface is not reachable, the protocol needs to return nullptr.
 *		
 *	See DMXProtocolArtNet for an example of a Protocol implementation.
 *
 */
class DMXPROTOCOL_API IDMXProtocol 
	: public IDMXProtocolBase
{

	///////////////////////////////////////////////////
	// ~Begin IDMXProtocol Interface definition

public:
	/**
	 * Whether protocol enabled
	 * @return Return true if the protocol is enabled
	 */
	virtual bool IsEnabled() const = 0;

	/**
	 * Get the Protocol Name
	 * @return Return FName of the protocol
	 */
	virtual const FName& GetProtocolName() const = 0;

	/**
	 * Get minimum supported universe ID for protocol
	 * @return		Minimum supported universe ID for protocol
	 */
	virtual int32 GetMinUniverseID() const = 0;

	/**
	 * Get maximum supported universe ID for protocol
	 * @return		Maximum supported universe ID for protocol
	 */
	virtual int32 GetMaxUniverseID() const = 0;

	/** Returns true if the Universe ID is valid for the protocol */
	virtual bool IsValidUniverseID(int32 UniverseID) const = 0;

	/** Returns a valid Universe ID as close as possible to the provided Universe ID */
	virtual int32 MakeValidUniverseID(int32 DesiredUniverseID) const = 0;

	/** 
	 * Returns the communication types the protocol supports for its input ports. 
	 * Can be empty if the protocol doesn't want to expose a selection to the user.
	 * If an empty array is returned, input ports that use the protocol will set CommunicationType to EDMXCommunicationType::InternalOnly.
	 *
	 * @return	Array of communication types the user can select from, when defining input ports in editor. 
	 */
	virtual const TArray<EDMXCommunicationType> GetInputPortCommunicationTypes() const = 0;

	/**
	 * Returns the communication types the protocol supports for its input ports.
	 * Can be empty if the protocol doesn't want to expose a selection to the user.
	 * If an empty array is returned, output ports that use the protocol will set CommunicationType to EDMXCommunicationType::InternalOnly.
	 *
	 * @return						Array of communication types the user can select from, when defining output ports in editor.
	 */
	virtual const TArray<EDMXCommunicationType> GetOutputPortCommunicationTypes() const = 0;


	/**
	 * Returns true if the Protocol supports a priority setting to filter inbound and outbound signals
	 * 
	 * @return						Whether the protocol supports Priority settings.
	 */
	virtual bool SupportsPrioritySettings() const = 0;

	/** 
	 * Called to register a DMXInputPort with the protocol. 
	 * This should be implemented so each port can only register once (see Art-Net for an example). 
	 *
	 * @param	InputPort			The input port that needs to be registered
	 * @return						True if the port was successfully registered
	 */
	virtual bool RegisterInputPort(const FDMXInputPortSharedRef& InputPort) = 0;

	/**
	 * Called to unregister a DMXInputPort with the protocol.
	 *
	 * @param	InputPort			The input port that needs to be unregistered
	 */
	virtual void UnregisterInputPort(const FDMXInputPortSharedRef& InputPort) = 0;

	/** 
	 * Called to register a DMXOutputPort with the protocol.
	 * If initially valid, returned sender needs to guarantee to be valid while the port is registered.
	 * This should be implemented so each port can only register once (see Art-Net for an example).
	 *
	 * @param	OutputPort			The Output port that needs to be registered
	 * @return						The DMX senders that can be used while the output port is registered.
	 */
	virtual TArray<TSharedPtr<IDMXSender>> RegisterOutputPort(const FDMXOutputPortSharedRef& OutputPort) = 0;

	/**
	 * Called to unregister a DMXOutputPort with the protocol.
	 *
	 * @param	OutputPort			The Output port that needs to be unregistered
	 */
	virtual void UnregisterOutputPort(const FDMXOutputPortSharedRef& OutputPort) = 0;

	/**
	 * Called to deduce if the communication type will be heard by corresponding input ports.
	 *
	 * As an example, this should be true, when the output port broadcasts to network. Broadcast traffic is heard
	 * by any input port that receives this protocol on corresponding universes. We forward this info to users,
	 * so they can take appropriate messures when specifying their ports.
	 *
	 * @param	InCommunicationType		The communication type that may cause intrinsic loopback
	 */
	virtual bool IsCausingLoopback(EDMXCommunicationType InCommunicationType) = 0;
	
	// ~End IDMXProtocol Interface definition
	///////////////////////////////////////////////////

public:
	static const TMap<FName, IDMXProtocolFactory*>& GetProtocolFactories()
	{
		static const FName DMXProtocolModuleName = TEXT("DMXProtocol");
		FDMXProtocolModule& DMXProtocolModule = FModuleManager::GetModuleChecked<FDMXProtocolModule>(DMXProtocolModuleName);
		return DMXProtocolModule.GetProtocolFactories();
	}

	static const TMap<FName, IDMXProtocolPtr>& GetProtocols()
	{
		static const FName DMXProtocolModuleName = TEXT("DMXProtocol");
		FDMXProtocolModule& DMXProtocolModule = FModuleManager::GetModuleChecked<FDMXProtocolModule>(DMXProtocolModuleName);
		return DMXProtocolModule.GetProtocols();
	}

	static const TArray<FName>& GetProtocolNames()
	{
		const TMap<FName, IDMXProtocolFactory*>& Protocols = GetProtocolFactories();
		static TArray<FName> ProtocolNames;
		Protocols.GenerateKeyArray(ProtocolNames);
		return ProtocolNames;
	}

	static FName GetFirstProtocolName()
	{
		const TMap<FName, IDMXProtocolFactory*>& ProtocolFactories = IDMXProtocol::GetProtocolFactories();

		for (const auto& Itt : ProtocolFactories)
		{
			return Itt.Key;
		}

		return FName();
	}

	/**
	 * If protocol exists return the pointer otherwise it create a new protocol first and then return the pointer.
	 * @param  InProtocolName Name of the requested protocol
	 * @return Return the pointer to protocol
	 */
	static IDMXProtocolPtr Get(const FName& ProtocolName = NAME_None)
	{
		static const FName DMXProtocolModuleName = TEXT("DMXProtocol");
		FDMXProtocolModule& DMXProtocolModule = FModuleManager::GetModuleChecked<FDMXProtocolModule>(DMXProtocolModuleName);
		return DMXProtocolModule.GetProtocol(ProtocolName);
	}
};
