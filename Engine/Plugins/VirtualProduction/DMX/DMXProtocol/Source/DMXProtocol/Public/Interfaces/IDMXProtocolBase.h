// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class DMXPROTOCOL_API IDMXProtocolBase
	: public TSharedFromThis<IDMXProtocolBase, ESPMode::ThreadSafe>
{
public:
	virtual ~IDMXProtocolBase() {}

	virtual bool Init() = 0;

	virtual bool Shutdown() = 0;

	virtual bool Tick(float DeltaTime) = 0;
};

