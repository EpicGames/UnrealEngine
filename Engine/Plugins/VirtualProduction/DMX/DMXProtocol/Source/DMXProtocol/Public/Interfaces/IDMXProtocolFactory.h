// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"

class DMXPROTOCOL_API IDMXProtocolFactory
{
public:
	virtual ~IDMXProtocolFactory() {}

	/**
	 * It creates only one instance of Protocol. There is the condition to check in DMXProtocolModule
	 * @return Return the pointer to protocol
	 */
	virtual IDMXProtocolPtr CreateProtocol(const FName& ProtocolName) = 0;
};
