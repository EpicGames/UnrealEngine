// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"

#include "Templates/SharedPointer.h"

class FDMXSignal;

/** */
class DMXPROTOCOL_API IDMXSender
	: public TSharedFromThis<IDMXSender>
{
public:
	virtual ~IDMXSender()
	{}

	/** Sends the DMX signal. Note, this may called concurrently from various threads */
	virtual void SendDMXSignal(const FDMXSignalSharedRef& DMXSignal) = 0;

	/** DEPRECATED 5.0 */
	UE_DEPRECATED(5.0, "Deprecated without replacement. Classes that implement IDMXSender do no longer need to implement this method.")
	virtual void ClearBuffer() {};
};
