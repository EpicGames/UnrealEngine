// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolArtNet.h"

#include "DMXProtocolArtNetConstants.h"
#include "DMXProtocolArtNetReceiver.h"
#include "DMXProtocolArtNetSender.h"
#include "Interfaces/IDMXSender.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"

#include "SocketSubsystem.h"


const TArray<EDMXCommunicationType> FDMXProtocolArtNet::InputPortCommunicationTypes = TArray<EDMXCommunicationType>(
	{ 
		EDMXCommunicationType::InternalOnly 
	});

const TArray<EDMXCommunicationType> FDMXProtocolArtNet::OutputPortCommunicationTypes = TArray<EDMXCommunicationType>(
	{ 
		EDMXCommunicationType::Broadcast, 
		EDMXCommunicationType::Unicast 
	});

FDMXProtocolArtNet::FDMXProtocolArtNet(const FName& InProtocolName)
	: ProtocolName(InProtocolName)	
{}

bool FDMXProtocolArtNet::Init()
{
	return true;
}

bool FDMXProtocolArtNet::Shutdown()
{
	return true;
}

bool FDMXProtocolArtNet::Tick(float DeltaTime)
{
	checkNoEntry();
	return true;
}

bool FDMXProtocolArtNet::IsEnabled() const
{
	return true;
}

const FName& FDMXProtocolArtNet::GetProtocolName() const
{
	return ProtocolName;
}

int32 FDMXProtocolArtNet::GetMinUniverseID() const
{
	return 0;
}

int32 FDMXProtocolArtNet::GetMaxUniverseID() const
{
	return ARTNET_MAX_UNIVERSE;
}

bool FDMXProtocolArtNet::IsValidUniverseID(int32 UniverseID) const
{
	return
		UniverseID >= ARTNET_MIN_UNIVERSE &&
		UniverseID <= ARTNET_MAX_UNIVERSE;
}

int32 FDMXProtocolArtNet::MakeValidUniverseID(int32 DesiredUniverseID) const
{
	return FMath::Clamp(DesiredUniverseID, static_cast<int32>(ARTNET_MIN_UNIVERSE), static_cast<int32>(ARTNET_MAX_UNIVERSE));
}

bool FDMXProtocolArtNet::SupportsPrioritySettings() const
{
	return false;
}

const TArray<EDMXCommunicationType> FDMXProtocolArtNet::GetInputPortCommunicationTypes() const
{
	return InputPortCommunicationTypes;
}

const TArray<EDMXCommunicationType> FDMXProtocolArtNet::GetOutputPortCommunicationTypes() const
{
	return OutputPortCommunicationTypes;
}

bool FDMXProtocolArtNet::RegisterInputPort(const FDMXInputPortSharedRef& InputPort)
{
	check(!InputPort->IsRegistered());
	check(!CachedInputPorts.Contains(InputPort));

	const FString& NetworkInterfaceAddress = InputPort->GetDeviceAddress();
	const EDMXCommunicationType CommunicationType = InputPort->GetCommunicationType();

	// Try to use an existing receiver or create a new one
	TSharedPtr<FDMXProtocolArtNetReceiver> Receiver = FindExistingReceiver(NetworkInterfaceAddress, CommunicationType);
	if (!Receiver.IsValid())
	{
		Receiver = FDMXProtocolArtNetReceiver::TryCreate(SharedThis(this), NetworkInterfaceAddress);
	}

	if (!Receiver.IsValid())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Could not create Art-Net receiver for input port %ls", *InputPort->GetPortName());
		
		return false;
	}

	Receivers.Add(Receiver);

	Receiver->AssignInputPort(InputPort);
	CachedInputPorts.Add(InputPort);
	
	return true;
}

void FDMXProtocolArtNet::UnregisterInputPort(const FDMXInputPortSharedRef& InputPort)
{
	check(CachedInputPorts.Contains(InputPort));
	CachedInputPorts.Remove(InputPort);

	for (const TSharedPtr<FDMXProtocolArtNetReceiver>& Receiver : TSet<TSharedPtr<FDMXProtocolArtNetReceiver>>(Receivers))
	{
		if (Receiver->ContainsInputPort(InputPort))
		{
			Receiver->UnassignInputPort(InputPort);

			if (Receiver->GetNumAssignedInputPorts() == 0)
			{
				Receivers.Remove(Receiver);
			}
		}
	}
}

TArray<TSharedPtr<IDMXSender>> FDMXProtocolArtNet::RegisterOutputPort(const FDMXOutputPortSharedRef& OutputPort)
{
	check(!OutputPort->IsRegistered());
	check(!CachedOutputPorts.Contains(OutputPort));

	const FString& NetworkInterfaceAddress = OutputPort->GetDeviceAddress();
	const EDMXCommunicationType CommunicationType = OutputPort->GetCommunicationType();

	// Try to use an existing receiver or create a new one
	TArray<TSharedPtr<IDMXSender>> NewSenders;
	if (CommunicationType == EDMXCommunicationType::Broadcast)
	{
		TSharedPtr<FDMXProtocolArtNetSender> NewSender = FindExistingBroadcastSender(NetworkInterfaceAddress);	
		if (!NewSender.IsValid())
		{
			NewSender = FDMXProtocolArtNetSender::TryCreateBroadcastSender(SharedThis(this), NetworkInterfaceAddress);
		}

		if (NewSender.IsValid())
		{
			NewSender->AssignOutputPort(OutputPort);
			NewSenders.Add(NewSender);
		}
	}
	else if (CommunicationType == EDMXCommunicationType::Unicast)
	{
		for (const FString& UnicastAddress : OutputPort->GetDestinationAddresses())
		{
			TSharedPtr<FDMXProtocolArtNetSender> NewSender = FindExistingUnicastSender(NetworkInterfaceAddress, UnicastAddress);
			if (!NewSender.IsValid())
			{
				NewSender = FDMXProtocolArtNetSender::TryCreateUnicastSender(SharedThis(this), NetworkInterfaceAddress, UnicastAddress);
			}

			if (NewSender.IsValid())
			{
				NewSender->AssignOutputPort(OutputPort);
				NewSenders.Add(NewSender);
			}
		}
	}
	else
	{
		UE_LOGF(LogDMXProtocol, Error, "Cannot create DMX Protocol Art-Net Sender. The communication type specified is not supported.");
	}

	if (NewSenders.Num() > 0)
	{
		CachedOutputPorts.Add(OutputPort);
	}
	else
	{
		UE_LOGF(LogDMXProtocol, Warning, "Could not create Art-Net sender for output port %ls", *OutputPort->GetPortName());
	}

	for (const TSharedPtr<IDMXSender>& Sender : NewSenders)
	{
		Senders.Add(StaticCastSharedPtr<FDMXProtocolArtNetSender>(Sender));
	}

	return NewSenders;
}

void FDMXProtocolArtNet::UnregisterOutputPort(const FDMXOutputPortSharedRef& OutputPort)
{
	check(CachedOutputPorts.Contains(OutputPort));
	CachedOutputPorts.Remove(OutputPort);

	for (const TSharedPtr<FDMXProtocolArtNetSender>& Sender : TSet<TSharedPtr<FDMXProtocolArtNetSender>>(Senders))
	{
		if (Sender->ContainsOutputPort(OutputPort))
		{
			Sender->UnassignOutputPort(OutputPort);

			if (Sender->GetNumAssignedOutputPorts() == 0)
			{
				Senders.Remove(Sender);
			}
		}
	}
}

bool FDMXProtocolArtNet::IsCausingLoopback(EDMXCommunicationType InCommunicationType)
{
	return InCommunicationType == EDMXCommunicationType::Broadcast;
}

TSharedPtr<FDMXProtocolArtNetSender> FDMXProtocolArtNet::FindExistingBroadcastSender(const FString& NetworkInterfaceAddress) const
{
	// Find the broadcast address
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

	TSharedRef<FInternetAddr> InternetAddr = SocketSubsystem->CreateInternetAddr();
	InternetAddr->SetBroadcastAddress();

	FString BroadcastAddress = InternetAddr->ToString(false);

	for (const TSharedPtr<FDMXProtocolArtNetSender>& Sender : Senders)
	{
		if (Sender->EqualsEndpoint(NetworkInterfaceAddress, BroadcastAddress))
		{
			return Sender;
		}
	}

	return nullptr;
}

TSharedPtr<FDMXProtocolArtNetSender> FDMXProtocolArtNet::FindExistingUnicastSender(const FString& NetworkInterfaceAddress, const FString& DestinationAddress) const
{
	for (const TSharedPtr<FDMXProtocolArtNetSender>& Sender : Senders)
	{
		if (Sender->EqualsEndpoint(NetworkInterfaceAddress, DestinationAddress))
		{
			return Sender;
		}
	}

	return nullptr;
}

TSharedPtr<FDMXProtocolArtNetReceiver> FDMXProtocolArtNet::FindExistingReceiver(const FString& IPAddress, EDMXCommunicationType CommunicationType) const
{
	for (const TSharedPtr<FDMXProtocolArtNetReceiver>& Receiver : Receivers)
	{
		if (Receiver->EqualsEndpoint(IPAddress))
		{
			return Receiver;
		}
	}

	return nullptr;
}
