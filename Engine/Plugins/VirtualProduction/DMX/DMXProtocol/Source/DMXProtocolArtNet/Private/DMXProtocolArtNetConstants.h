// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DMXProtocolTypes.h"

enum
{
	LOW_BYTE = 0x00FF,
	HIGH_BYTE = 0xFF00
};

#ifndef DMX_SHORT_GET_HIGH_BIT
#define DMX_SHORT_GET_HIGH_BIT(x) ((HIGH_BYTE & x) >> 8)
#endif

#ifndef DMX_SHORT_GET_LOW_BYTE
#define DMX_SHORT_GET_LOW_BYTE(x)  (LOW_BYTE & x)
#endif

#define DMX_PROTOCOLNAME_ARTNET "Art-Net"

enum
{
	ARTNET_LONG_NAME_LENGTH = 64,
	ARTNET_MAX_PORTS = 4,
	ARTNET_MAX_RDM_ADDRESS_COUNT = 32,
	ARTNET_MAX_RDM_DATA = 512,
	ARTNET_MAX_UID_COUNT = 200,
	ARTNET_REPORT_LENGTH = 64,
	ARTNET_SHORT_NAME_LENGTH = 18,
	ARTNET_UNIVERSE_ADDRESS = 14
};

enum {
	ARTNET_POLL = 0x2000,
	ARTNET_REPLY = 0x2100,
	ARTNET_DMX = 0x5000,
	ARTNET_ADDRESS = 0x6000,
	ARTNET_INPUT = 0x7000,
	ARTNET_TODREQUEST = 0x8000,
	ARTNET_TODDATA = 0x8100,
	ARTNET_TODCONTROL = 0x8200,
	ARTNET_RDM = 0x8300,
	ARTNET_VIDEOSTEUP = 0xa010,
	ARTNET_VIDEOPALETTE = 0xa020,
	ARTNET_VIDEODATA = 0xa040,
	ARTNET_MACMASTER = 0xf000,
	ARTNET_MACSLAVE = 0xf100,
	ARTNET_FIRMWAREMASTER = 0xf200,
	ARTNET_FIRMWAREREPLY = 0xf300,
	ARTNET_IPPROG = 0xf800,
	ARTNET_IPREPLY = 0xf900,
	ARTNET_MEDIA = 0x9000,
	ARTNET_MEDIAPATCH = 0x9200,
	ARTNET_MEDIACONTROLREPLY = 0x9300
};

enum 
{
	ARTNET_DMX_LENGTH = 512,
	ARTNET_DMX_PACKAGE_SIZE = 530,
	ARTNET_POLL_PACKAGE_SIZE = 14,
};

enum
{
	ARTNET_MIN_UNIVERSE = 0,
	ARTNET_MAX_UNIVERSE = 32767
};

enum  
{
	ARTNET_TTM_DEFAULT = 0xFF,  /**< default, ArtPollReplies are broadcast, and nodes won't send a ArtPollReply when conditions change */
	ARTNET_TTM_PRIVATE = 0xFE,  /**< ArtPollReplies aren't broadcast */
	ARTNET_TTM_noauto = 0xFD    /**< ArtPollReplies are send when node conditions chang */
};

// TOD actions
typedef enum {
	ARTNET_TOD_FULL = 0x00,
	ARTNET_TOD_FLUSH = 0x01,
} artnet_tod_command_code;


// RDM
enum
{
	ARTNET_RDM_VERSION = 0x01,
	ARTNET_MAX_RDM_ADCOUNT = 32,

	/**
	 * ARTNET_RDM_UID_WIDTH are 6 bytes, the first two bytes are the
	 * [manufacturer code](http://tsp.esta.org/tsp/working_groups/CP/mfctrIDs.php)
	 * and the last 4 bytes are the device id. UIDs are written as: XXXX:YYYYYYYY
	 */
	ARTNET_RDM_UID_WIDTH = 6,
};

// RDM functions
enum
{
	ARTNET_RDM_PROCESS_PACKET = 0x00
};

enum
{
	ARTNET_ADDRESS_NO_CHANGE = 0x7f,
	ARTNET_PORT = 6454,
	ARTNET_SENDER_PORT = 3330,
	ARTNET_STRING_SIZE = 8,
	ARTNET_VERSION = 14,
	OEM_HI = 0x04,
	OEM_LO = 0x30,
	ESTA_HI = 'z',
	ESTA_LO = 'p',
	TTM_BEHAVIOUR_MASK = 0x02,
	TTM_REPLY_MASK = 0x01,
	PROGRAM_NO_CHANGE = 0x7f,
	PROGRAM_DEFAULTS = 0x00,
	PROGRAM_CHANGE_MASK = 0x80,
	STATUS_PROG_AUTH_MASK = 0x30,
	PORT_STATUS_LPT_MODE = 0x02,
	PORT_STATUS_SHORT = 0x04,
	PORT_STATUS_ERROR = 0x04,
	PORT_STATUS_DISABLED_MASK = 0x08,
	PORT_STATUS_MERGE = 0x08,
	PORT_STATUS_DMXEXT = 0x10,
	PORT_STATUS_DMX_SIP = 0x20,
	PORT_STATUS_DMXEST = 0x40,
	PORT_STATUS_ACT_MASK = 0x80,
	PORT_DISABLE_MASK = 0x01,
	TOD_RESPONSE_FULL = 0x00,
	TOD_RESPONSE_NAK = 0x00,
	MIN_PACKET_SIZE = 10,
	MERGEIMEOUT_SECONDS = 10,
	FIRMWAREIMEOUT_SECONDS = 20,
	RECV_NO_DATA = 1,
	MAX_NODE_BCAST_LIMIT = 30 // always bcast after this point
};
