// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolArtNetModule.h"

#include "DMXProtocolArtNet.h"
#include "DMXProtocolTypes.h"
#include "DMXProtocolArtNetConstants.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXPortManager.h"

#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDMXProtocolArtNetModule, DMXProtocolArtNet);


const FName FDMXProtocolArtNetModule::NAME_Artnet = FName(DMX_PROTOCOLNAME_ARTNET);

FAutoConsoleCommand FDMXProtocolArtNetModule::SendDMXCommand(
	TEXT("DMX.ArtNet.SendDMX"),
	TEXT("Command for sending DMX through ArtNet Protocol. DMX.ArtNet.SendDMX [UniverseID] Channel:Value Channel:Value Channel:Value n\t DMX.ArtNet.SendDMX 17 10:6 11:7 12:8 13:9 n\t It will send channels values to the DMX to Universe 17"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FDMXProtocolArtNetModule::SendDMXCommandHandler)
);

FAutoConsoleCommand FDMXProtocolArtNetModule::ResetDMXSendUniverseCommand(
	TEXT("DMX.ArtNet.ResetDMXSend"),
	TEXT("Command for resetting DMX universe values."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FDMXProtocolArtNetModule::ResetDMXSendUniverseHandler)
);


IDMXProtocolPtr FDMXProtocolFactoryArtNet::CreateProtocol(const FName & ProtocolName)
{
	IDMXProtocolPtr ProtocolArtNetPtr = MakeShared<FDMXProtocolArtNet, ESPMode::ThreadSafe>(ProtocolName);
	if (ProtocolArtNetPtr->IsEnabled())
	{
		if (!ProtocolArtNetPtr->Init())
		{
			UE_LOG_DMXPROTOCOL(Verbose, TEXT("ArtNet failed to initialize!"));
			ProtocolArtNetPtr->Shutdown();
			ProtocolArtNetPtr = nullptr;
		}
	}
	else
	{
		UE_LOG_DMXPROTOCOL(Verbose, TEXT("ArtNet disabled!"));
		ProtocolArtNetPtr->Shutdown();
		ProtocolArtNetPtr = nullptr;
	}

	return ProtocolArtNetPtr;
}

void FDMXProtocolArtNetModule::StartupModule()
{
	FactoryArtNet = MakeUnique<FDMXProtocolFactoryArtNet>();

	// Bind to the protocol module requesting registration of protocols
	FDMXProtocolModule::GetOnRequestProtocolRegistration().AddRaw(this, &FDMXProtocolArtNetModule::RegisterWithProtocolModule);
}

void FDMXProtocolArtNetModule::ShutdownModule()
{
	FDMXProtocolModule::GetOnRequestProtocolRegistration().RemoveAll(this);

	// Unregister and destroy protocol
	FDMXProtocolModule* DMXProtocolModule = FModuleManager::GetModulePtr<FDMXProtocolModule>("DMXProtocol");
	if (DMXProtocolModule != nullptr)
	{
		DMXProtocolModule->UnregisterProtocol(DMX_PROTOCOLNAME_ARTNET);
	}
	
	FactoryArtNet.Reset();
}

FDMXProtocolArtNetModule& FDMXProtocolArtNetModule::Get()
{
	return FModuleManager::GetModuleChecked<FDMXProtocolArtNetModule>("DMXProtocolArtNet");
}

void FDMXProtocolArtNetModule::RegisterWithProtocolModule(TArray<FDMXProtocolRegistrationParams>& InOutProtocolRegistrationParamsArray)
{	
	FDMXProtocolRegistrationParams RegistrationParams;
	RegistrationParams.ProtocolName = DMX_PROTOCOLNAME_ARTNET;
	RegistrationParams.ProtocolFactory = FactoryArtNet.Get();

	InOutProtocolRegistrationParamsArray.Add(RegistrationParams);
}

void FDMXProtocolArtNetModule::SendDMXCommandHandler(const TArray<FString>& Args)
{
	if (Args.Num() < 2)
	{
		UE_LOG_DMXPROTOCOL(Verbose, TEXT("Not enough arguments. It won't be sent\n"
										"Command structure is DMX.ArtNet.SendDMX [UniverseID] Channel:Value Channel:Value Channel:Value\n"
										"For example: DMX.ArtNet.SendDMX 17 10:6 11:7 12:8 13:9"));
		return;
	}

	uint32 UniverseID = 0;
	LexTryParseString<uint32>(UniverseID, *Args[0]);
	if (UniverseID > ARTNET_MAX_UNIVERSE)
	{
		UE_LOG_DMXPROTOCOL(Verbose, TEXT("The UniverseID is bigger then the max universe value. It won't be sent\n"
			"For example: DMX.ArtNet.SendDMX 17 10:6 11:7 12:8 13:9\n"
			"Where Universe %d should be less then %d"), UniverseID, ARTNET_MAX_UNIVERSE);
		return;
	}

	TMap<int32, uint8> ChannelToValueMap;
	for (int32 i = 1; i < Args.Num(); i++)
	{
		FString ChannelAndValue = Args[i];

		FString KeyStr;
		FString ValueStr;
		ChannelAndValue.Split(TEXT(":"), &KeyStr, &ValueStr);

		uint32 Key = 0;
		uint32 Value = 0;
		LexTryParseString<uint32>(Key, *KeyStr);
		if (Key > DMX_UNIVERSE_SIZE)
		{
			UE_LOG_DMXPROTOCOL(Verbose, TEXT("The input channel is bigger then the universe size. It won't be sent\n"
				"For example: DMX.ArtNet.SendDMX 17 10:6 11:7 12:8 13:9\n"
				"Where channel %d should be less then %d"), Key, DMX_UNIVERSE_SIZE);
			return;
		}
		LexTryParseString<uint32>(Value, *ValueStr);
		if (Value > DMX_MAX_CHANNEL_VALUE)
		{
			UE_LOG_DMXPROTOCOL(Verbose, TEXT("The input value is bigger then the universe size. It won't be sent\n"
				"For example: DMX.ArtNet.SendDMX 17 10:6 11:7 12:8 13:9\n"
				"Where value %d should be less then %d"), Value, DMX_MAX_CHANNEL_VALUE);
			return;
		}
		ChannelToValueMap.Add(Key, Value);
	}

	for (const FDMXOutputPortSharedRef& OutputPort : FDMXPortManager::Get().GetOutputPorts())
	{
		if (OutputPort->GetProtocol()->GetProtocolName() == DMX_PROTOCOLNAME_ARTNET)
		{
			OutputPort->SendDMX(UniverseID, ChannelToValueMap);
		}
	}
}

void FDMXProtocolArtNetModule::ResetDMXSendUniverseHandler(const TArray<FString>& Args)
{
	if (Args.Num() < 1)
	{
		UE_LOG_DMXPROTOCOL(Verbose, TEXT("Not enough arguments. It won't be sent\n"
			"Command structure is DMX.ArtNet.ResetDMXSend [UniverseID]"));
		return;
	}

	uint32 UniverseID = 0;
	LexTryParseString<uint32>(UniverseID, *Args[0]);
	if (UniverseID > ARTNET_MAX_UNIVERSE)
	{
		UE_LOG_DMXPROTOCOL(Verbose, TEXT("The UniverseID is bigger then the max universe value. It won't be sent. It won't be sent\n"
			"Where Universe %d should be less then %d"), UniverseID, ARTNET_MAX_UNIVERSE);
		return;
	}

	// Create Channel To Value map with all channels being set to 0
	TMap<int32, uint8> ChannelToValueMap;
	for (int ChannelID = 1; ChannelID <= DMX_MAX_ADDRESS; ChannelID++)
	{
		ChannelToValueMap.Add(ChannelID, 0);
	}

	for (const FDMXOutputPortSharedRef& OutputPort : FDMXPortManager::Get().GetOutputPorts())
	{
		if (OutputPort->GetProtocol()->GetProtocolName() == DMX_PROTOCOLNAME_ARTNET)
		{
			OutputPort->SendDMX(UniverseID, ChannelToValueMap);
		}
	}
}
