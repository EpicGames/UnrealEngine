// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolArtNetReceiver.h"

#include "DMXProtocolLog.h"
#include "DMXProtocolArtNet.h"
#include "DMXProtocolArtNetConstants.h"
#include "DMXProtocolUtils.h"
#include "DMXStats.h"
#include "IO/DMXInputPort.h"
#include "Packets/DMXProtocolArtNetPackets.h"

#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Common/UdpSocketBuilder.h"
#include "HAL/RunnableThread.h"
#include "Serialization/ArrayReader.h"

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Art-Net Packages Received Total"), STAT_ArtNetPackagesReceived, STATGROUP_DMX);


FDMXProtocolArtNetReceiver::FDMXProtocolArtNetReceiver(const TSharedPtr<FDMXProtocolArtNet, ESPMode::ThreadSafe>& InArtNetProtocol, FSocket& InSocket, TSharedRef<FInternetAddr> InEndpointInternetAddr)
	: Protocol(InArtNetProtocol)
	, Socket(&InSocket)
	, EndpointInternetAddr(InEndpointInternetAddr)
	, bStopping(false)
	, Thread(nullptr)
{
	check(Socket->GetSocketType() == SOCKTYPE_Datagram);

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	ReceivedSenderInternetAddr = SocketSubsystem->CreateInternetAddr();

	FString ReceiverThreadName = FString(TEXT("ArtNetReceiver_")) + InEndpointInternetAddr->ToString(true);
	Thread = FRunnableThread::Create(this, *ReceiverThreadName, 0, TPri_TimeCritical, FPlatformAffinity::GetPoolThreadMask());

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Created Art-Net Receiver at %ls", *EndpointInternetAddr->ToString(false));
}

FDMXProtocolArtNetReceiver::~FDMXProtocolArtNetReceiver()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}

	if (Socket)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		SocketSubsystem->DestroySocket(Socket);
	}

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Destroyed Art-Net Receiver at %ls", *EndpointInternetAddr->ToString(false));
}

TSharedPtr<FDMXProtocolArtNetReceiver> FDMXProtocolArtNetReceiver::TryCreate(const TSharedPtr<FDMXProtocolArtNet, ESPMode::ThreadSafe>& ArtNetProtocol, const FString& IPAddress)
{
	TSharedPtr<FInternetAddr> EndpointInternetAddr = FDMXProtocolUtils::CreateInternetAddr(IPAddress, ARTNET_PORT);
	if (!EndpointInternetAddr.IsValid())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create Art-Net receiver: Invalid IP address: %ls", *IPAddress);
		return nullptr;
	}

	FIPv4Endpoint Endpoint = FIPv4Endpoint(EndpointInternetAddr);

	FSocket* NewListeningSocket = FUdpSocketBuilder(TEXT("UDPArtNetListeningSocket"))
		.AsNonBlocking()
		.AsReusable()
		.BoundToEndpoint(Endpoint);

	if (!NewListeningSocket)
	{
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create Art-Net receiver: Error create ListeningSocket for: %ls", *IPAddress);
		return nullptr;
	}

	TSharedPtr<FDMXProtocolArtNetReceiver> NewReceiver = MakeShareable(new FDMXProtocolArtNetReceiver(ArtNetProtocol, *NewListeningSocket, EndpointInternetAddr.ToSharedRef()));

	return NewReceiver;
}

bool FDMXProtocolArtNetReceiver::EqualsEndpoint(const FString& IPAddress) const
{
	TSharedPtr<FInternetAddr> OtherEndpointInternetAddr = FDMXProtocolUtils::CreateInternetAddr(IPAddress, ARTNET_PORT);
	if (OtherEndpointInternetAddr.IsValid() && OtherEndpointInternetAddr->CompareEndpoints(*EndpointInternetAddr))
	{
		return true;
	}

	return false;
}

void FDMXProtocolArtNetReceiver::AssignInputPort(const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort)
{
	check(!AssignedInputPorts.Contains(InputPort));

	const FScopeLock LockInputPorts(&AccessInputPortsMutex);
	AssignedInputPorts.Add(InputPort);
}

void FDMXProtocolArtNetReceiver::UnassignInputPort(const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort)
{
	check(AssignedInputPorts.Contains(InputPort));

	const FScopeLock LockInputPorts(&AccessInputPortsMutex);
	AssignedInputPorts.Remove(InputPort);
}

bool FDMXProtocolArtNetReceiver::Init()
{
	return true;
}

uint32 FDMXProtocolArtNetReceiver::Run()
{
	// No receive refresh rate, it would deter the timestamp
	
	while (!bStopping)
	{
		Update(FTimespan::FromMilliseconds(1000.f));
	}

	return 0;
}

void FDMXProtocolArtNetReceiver::Stop()
{
	bStopping = true;
}

void FDMXProtocolArtNetReceiver::Exit()
{
}

void FDMXProtocolArtNetReceiver::Tick()
{
	Update(FTimespan::Zero());
}

FSingleThreadRunnable* FDMXProtocolArtNetReceiver::GetSingleThreadInterface()
{
	return this;
}

void FDMXProtocolArtNetReceiver::Update(const FTimespan& SocketWaitTime)
{
	if (!Socket->Wait(ESocketWaitConditions::WaitForRead, SocketWaitTime))
	{
		return;
	}

	uint32 Size = 0;
	int32 Read = 0;
	while (Socket->HasPendingData(Size))
	{
		const TSharedRef<FArrayReader> Reader = MakeShared<FArrayReader>(true);

		// Use an aligned size instead of ART_NET_PACKAGE_SIZE
		constexpr uint32 ArtNetMaxReaderSize = 1024u;
		Reader->SetNumUninitialized(FMath::Min(Size, ArtNetMaxReaderSize));
		
        if (Socket->RecvFrom(Reader->GetData(), Reader->Num(), Read, *ReceivedSenderInternetAddr))
		{
            Reader->RemoveAt(Read, Reader->Num() - Read, EAllowShrinking::No);
			
			DistributeReceivedData(Reader);
		}
	}
}

void FDMXProtocolArtNetReceiver::DistributeReceivedData(const TSharedRef<FArrayReader>& PacketReader)
{
#if UE_BUILD_DEBUG
	check(Protocol.IsValid());
#endif

	switch (GetPacketOpCode(PacketReader))
	{
	case ARTNET_POLL:
		HandlePool(PacketReader);
		break;
	case ARTNET_REPLY:
		HandleReplyPacket(PacketReader);
		break;
	case ARTNET_DMX:
		HandleDataPacket(PacketReader);
		break;
	case ARTNET_TODREQUEST:
		HandleTodRequest(PacketReader);
		break;
	case ARTNET_TODDATA:
		HandleTodData(PacketReader);
		break;
	case ARTNET_TODCONTROL:
		HandleTodControl(PacketReader);
		break;
	case ARTNET_RDM:
		HandleRdm(PacketReader);
	default:
		break;
	}
}

uint16 FDMXProtocolArtNetReceiver::GetPacketOpCode(const TSharedRef<FArrayReader>& Buffer) const
{
	uint16 OpCode = 0x0000;
	constexpr uint32 MinCheck = ARTNET_STRING_SIZE + 2;
	if (Buffer->Num() > MinCheck)
	{
		// Get OpCode
		Buffer->Seek(ARTNET_STRING_SIZE);
		*Buffer << OpCode;

		// Reset Position
		Buffer->Seek(0);
	}

	return OpCode;
}

void FDMXProtocolArtNetReceiver::HandleDataPacket(const TSharedRef<FArrayReader>& PacketReader)
{
	uint16 UniverseID = 0x0000;
	constexpr uint32 MinCheck = ARTNET_UNIVERSE_ADDRESS + 2;
	if (PacketReader->Num() > MinCheck)
	{
		// Get OpCode
		PacketReader->Seek(ARTNET_UNIVERSE_ADDRESS);
		*PacketReader << UniverseID;

		// Reset Position
		PacketReader->Seek(0);

		FDMXProtocolArtNetDMXPacket ArtNetDMXPacket;
		*PacketReader << ArtNetDMXPacket;

		constexpr int32 Priority = 0;
		FDMXSignalSharedRef DMXSignal = MakeShared<FDMXSignal, ESPMode::ThreadSafe>(FPlatformTime::Seconds(), UniverseID, Priority, TArray<uint8>(ArtNetDMXPacket.Data, DMX_UNIVERSE_SIZE));

		AccessInputPortsMutex.Lock();
		for (const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort : AssignedInputPorts)
		{				
			InputPort->InputDMXSignal(DMXSignal);
		}
		AccessInputPortsMutex.Unlock();

		INC_DWORD_STAT(STAT_ArtNetPackagesReceived);
	}
}

void FDMXProtocolArtNetReceiver::HandlePool(const TSharedRef<FArrayReader>& PacketReader)
{
	// Packet type recognized but handler is not implemented
}

void FDMXProtocolArtNetReceiver::HandleReplyPacket(const TSharedRef<FArrayReader>& PacketReader)
{
	// Packet type recognized but handler is not implemented
}

void FDMXProtocolArtNetReceiver::HandleTodRequest(const TSharedRef<FArrayReader>& PacketReader)
{
	// Packet type recognized but handler is not implemented
}

void FDMXProtocolArtNetReceiver::HandleTodData(const TSharedRef<FArrayReader>& PacketReader)
{
	// Packet type recognized but handler is not implemented
}

void FDMXProtocolArtNetReceiver::HandleTodControl(const TSharedRef<FArrayReader>& PacketReader)
{
	// Packet type recognized but handler is not implemented
}

void FDMXProtocolArtNetReceiver::HandleRdm(const TSharedRef<FArrayReader>& PacketReader)
{
	// Packet type recognized but handler is not implemented
}
