// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"

#include "CoreMinimal.h"
#include "HAL/Runnable.h"
#include "Misc/ScopeLock.h"
#include "Misc/SingleThreadRunnable.h"
#include "Templates/Atomic.h"

class FDMXInputPort;
class FDMXProtocolArtNet;

class FArrayReader;
class FSocket;
class FRunnableThread;
class ISocketSubsystem;
class FInternetAddr;


class FDMXProtocolArtNetReceiver
	: public FRunnable
	, public FSingleThreadRunnable
{
protected:
	/** Constructor. Hidden on purpose, use Create instead. */
	FDMXProtocolArtNetReceiver(const TSharedPtr<FDMXProtocolArtNet, ESPMode::ThreadSafe>& InArtNetProtocol, FSocket& InSocket, TSharedRef<FInternetAddr> InEndpointInternetAddr);

public:
	/** Destructor */
	virtual ~FDMXProtocolArtNetReceiver();

	/** 
	 * Creates a new receiver for the specified IP address. returns null if no receiver can be created.
	 * Note: Doesn't test if another receiver on same IP already exists. Use HandlesInputPort to test other instances.
	 * If another receiver exists that handles IPAddress, reuse that instead.
	 */
	static TSharedPtr<FDMXProtocolArtNetReceiver> TryCreate(const TSharedPtr<FDMXProtocolArtNet, ESPMode::ThreadSafe>& ArtNetProtocol, const FString& IPAddress);

public:
	/** Returns true if the receiver uses specified IP address */
	bool EqualsEndpoint(const FString& IPAddress) const;

	/** Assigns an input port to be handled by this receiver */
	void AssignInputPort(const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort);

	/** Unassigns an input port from this receiver */
	void UnassignInputPort(const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort);

	/** Returns true if the input port is currently assigned to this receiver */
	bool ContainsInputPort(const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort) const { return AssignedInputPorts.Contains(InputPort); }

	/** Gets the num input ports currently assigned to this receiver */
	int32 GetNumAssignedInputPorts() const { return AssignedInputPorts.Num(); }

	/** Returns the input ports assigned to the receiver */
	const TSet<TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>>& GetAssignedInputPorts() const { return AssignedInputPorts; }

private:
	/** The input ports the receiver uses */
	TSet<TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>> AssignedInputPorts;

	/** Mutex access to the input ports array */
	FCriticalSection AccessInputPortsMutex;

protected:
	//~ Begin FRunnable implementation
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;
	//~ End FRunnable implementation

	//~ Begin FSingleThreadRunnable implementation
	virtual void Tick() override;
	virtual class FSingleThreadRunnable* GetSingleThreadInterface() override;
	//~ End FSingleThreadRunnable implementation

protected:
	/** 
	 * Updates the thread with data from the socket.  
	 * The socket will wait for specified time unless data is received.
	 */
	void Update(const FTimespan& SocketWaitTime);

	/** Distributes received data to the protocol to be processed further */
	void DistributeReceivedData(const TSharedRef<FArrayReader>& PacketReader);

	/** Get package 2 bytes ID from the buffer, as it was received */
	uint16 GetPacketOpCode(const TSharedRef<FArrayReader>& PacketReader) const;

	/** Handles a received Art-Net data packet */
	void HandleDataPacket(const TSharedRef<FArrayReader>& PacketReader);

protected:
	// Handlers for recognized but not implemented packages
	void HandlePool(const TSharedRef<FArrayReader>& PacketReader);
	void HandleReplyPacket(const TSharedRef<FArrayReader>& PacketReader);
	void HandleTodRequest(const TSharedRef<FArrayReader>& PacketReader);
	void HandleTodData(const TSharedRef<FArrayReader>& PacketReader);
	void HandleTodControl(const TSharedRef<FArrayReader>& PacketReader);
	void HandleRdm(const TSharedRef<FArrayReader>& PacketReader);

private:
	/** The owning Art-Net protocol */
	TSharedPtr<FDMXProtocolArtNet, ESPMode::ThreadSafe> Protocol;

	/** The network socket. */
	FSocket* Socket = nullptr;

	/** The endpoint internet addr (usually the network interface card IP Address) */
	TSharedPtr<FInternetAddr> EndpointInternetAddr;

	/** The sender when packets were received */
	TSharedPtr<FInternetAddr> ReceivedSenderInternetAddr;

	/** Flag indicating that the thread is stopping. */
	TAtomic<bool> bStopping;

	/** The thread object. */
	FRunnableThread* Thread = nullptr;

	/** The receiver thread's name. */
	FString ThreadName;
};
