// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolArtNetSender.h"

#include "DMXProtocolArtNet.h"
#include "DMXProtocolConstants.h"
#include "DMXProtocolLog.h"
#include "DMXProtocolSettings.h"
#include "DMXProtocolUtils.h"
#include "DMXStats.h"
#include "Packets/DMXProtocolArtNetPackets.h"

#include "IMessageAttachment.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Common/UdpSocketBuilder.h"
#include "Common/UdpSocketSender.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "UObject/Class.h"

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Art-Net Packages Sent Total"), STAT_ArtNetPackagesSent, STATGROUP_DMX);


FDMXProtocolArtNetSender::FDMXProtocolArtNetSender(const TSharedPtr<FDMXProtocolArtNet, ESPMode::ThreadSafe>& InArtNetProtocol, FSocket& InSocket, TSharedRef<FInternetAddr> InNetworkInterfaceInternetAddr, TSharedRef<FInternetAddr> InDestinationInternetAddr)
	: Protocol(InArtNetProtocol)
	, Socket(&InSocket)
	, NetworkInterfaceInternetAddr(InNetworkInterfaceInternetAddr)
	, DestinationInternetAddr(InDestinationInternetAddr)
{
	check(DestinationInternetAddr.IsValid());

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Created Art-Net Sender at %ls sending to %ls", *NetworkInterfaceInternetAddr->ToString(false), *DestinationInternetAddr->ToString(false));
}

FDMXProtocolArtNetSender::~FDMXProtocolArtNetSender()
{
	if (Socket)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		SocketSubsystem->DestroySocket(Socket);
	}

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Destroyed Art-Net Sender at %ls sending to %ls", *NetworkInterfaceInternetAddr->ToString(false), *DestinationInternetAddr->ToString(false));
}

TSharedPtr<FDMXProtocolArtNetSender> FDMXProtocolArtNetSender::TryCreateUnicastSender(
	const TSharedPtr<FDMXProtocolArtNet,
	ESPMode::ThreadSafe>& ArtNetProtocol,
	const FString& InNetworkInterfaceIP,
	const FString& InUnicastIP)
{
	// Try to create a socket
	TSharedPtr<FInternetAddr> NewNetworkInterfaceInternetAddr = FDMXProtocolUtils::CreateInternetAddr(InNetworkInterfaceIP, ARTNET_SENDER_PORT);
	if (!NewNetworkInterfaceInternetAddr.IsValid())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create Art-Net sender: Invalid IP address: %ls", *InNetworkInterfaceIP);
		return nullptr;
	}

	FIPv4Endpoint NewNetworkInterfaceEndpoint = FIPv4Endpoint(NewNetworkInterfaceInternetAddr);
	
	FSocket* NewSocket =
		FUdpSocketBuilder(TEXT("UDPArtNetUnicastSocket"))
		.AsBlocking()
		.AsReusable()
		.BoundToEndpoint(NewNetworkInterfaceEndpoint);
	
	if(!NewSocket)
	{
		UE_LOGF(LogDMXProtocol, Warning, "Invalid Network Interface IP %ls for DMX Port. Please update your Output Port in Project Settings -> Plugins -> DMX Plugin", *InNetworkInterfaceIP);
		return nullptr;
	}

	// Try create the unicast internet addr
	TSharedPtr<FInternetAddr> NewUnicastInternetAddr = FDMXProtocolUtils::CreateInternetAddr(InUnicastIP, ARTNET_PORT);
	if (!NewUnicastInternetAddr.IsValid())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Invalid Unicast IP %ls for DMX Port. Please update your Output Port in Project Settings -> Plugins -> DMX Plugin", *InUnicastIP);
		return nullptr;
	}

	TSharedPtr<FDMXProtocolArtNetSender> NewSender = MakeShareable(new FDMXProtocolArtNetSender(ArtNetProtocol, *NewSocket, NewNetworkInterfaceInternetAddr.ToSharedRef(), NewUnicastInternetAddr.ToSharedRef()));

	return NewSender;
}

TSharedPtr<FDMXProtocolArtNetSender> FDMXProtocolArtNetSender::TryCreateBroadcastSender(
	const TSharedPtr<FDMXProtocolArtNet,
	ESPMode::ThreadSafe>& ArtNetProtocol,
	const FString& InNetworkInterfaceIP)
{
	// Try to create a socket
	TSharedPtr<FInternetAddr> NewNetworkInterfaceInternetAddr = FDMXProtocolUtils::CreateInternetAddr(InNetworkInterfaceIP, ARTNET_SENDER_PORT);
	if (!NewNetworkInterfaceInternetAddr.IsValid())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create Art-Net sender: Invalid IP address: %ls", *InNetworkInterfaceIP);
		return nullptr;
	}

	FIPv4Endpoint NewNetworkInterfaceEndpoint = FIPv4Endpoint(NewNetworkInterfaceInternetAddr);

	FSocket* NewSocket = 
		FUdpSocketBuilder(TEXT("UDPArtNetBroadcastSocket"))
		.AsReusable()
		.AsBlocking()
		.WithBroadcast()
		.BoundToEndpoint(NewNetworkInterfaceEndpoint);

	if(!NewSocket)
	{
		UE_LOGF(LogDMXProtocol, Warning, "Invalid Network Interface IP %ls for DMX Port. Please update your Output Ports in Project Settings -> Plugins -> DMX Plugin", *InNetworkInterfaceIP);
		return nullptr;
	}

	// Try create the broadcast internet addr
	TSharedRef<FInternetAddr> NewBroadcastInternetAddr = CreateBroadcastInternetAddr(ARTNET_PORT);

	TSharedPtr<FDMXProtocolArtNetSender> NewSender = MakeShareable(new FDMXProtocolArtNetSender(ArtNetProtocol, *NewSocket, NewNetworkInterfaceInternetAddr.ToSharedRef(), NewBroadcastInternetAddr));

	return NewSender;
}

bool FDMXProtocolArtNetSender::EqualsEndpoint(const FString& NetworkInterfaceIP, const FString& DestinationIPAddress) const
{
	TSharedPtr<FInternetAddr> OtherNetworkInterfaceInternetAddr = FDMXProtocolUtils::CreateInternetAddr(NetworkInterfaceIP, ARTNET_SENDER_PORT);
	if (OtherNetworkInterfaceInternetAddr.IsValid() && OtherNetworkInterfaceInternetAddr->CompareEndpoints(*NetworkInterfaceInternetAddr))
	{
		TSharedPtr<FInternetAddr> OtherDestinationInternetAddr = FDMXProtocolUtils::CreateInternetAddr(DestinationIPAddress, ARTNET_PORT);
		if (OtherDestinationInternetAddr.IsValid() && OtherDestinationInternetAddr->CompareEndpoints(*DestinationInternetAddr))
		{
			return true;
		}
	}

	return false;
}

void FDMXProtocolArtNetSender::AssignOutputPort(const TSharedPtr<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort)
{
	check(!AssignedOutputPorts.Contains(OutputPort));
	AssignedOutputPorts.Add(OutputPort);
}

void FDMXProtocolArtNetSender::UnassignOutputPort(const TSharedPtr<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort)
{
	check(AssignedOutputPorts.Contains(OutputPort));
	AssignedOutputPorts.Remove(OutputPort);
}

bool FDMXProtocolArtNetSender::IsCausingLoopback() const
{
	return CommunicationType == EDMXCommunicationType::Broadcast;
}

void FDMXProtocolArtNetSender::SendDMXSignal(const FDMXSignalSharedRef& DMXSignal)
{
	// This may be called from multiple threads, so calls need be synchronized
	const FScopeLock SendDMXLock(&SendDMXCriticalSection);

	FDMXProtocolArtNetDMXPacket ArtNetDMXPacket;
	FMemory::Memcpy(ArtNetDMXPacket.Data, DMXSignal->ChannelData.GetData(), ARTNET_DMX_LENGTH);

	uint16 UniverseID = static_cast<uint16>(DMXSignal->ExternUniverseID);

	//Set Packet Data
	ArtNetDMXPacket.Physical = 0; // As per Standard: For information only. We always specify port 0.
	ArtNetDMXPacket.Universe = UniverseID;
	ArtNetDMXPacket.Sequence = 0x00; // As per Standard: The Sequence field is set to 0x00 to disable this feature.

	TSharedPtr<FBufferArchive> BufferArchive = ArtNetDMXPacket.Pack(ARTNET_DMX_LENGTH);

	int32 SendDataSize = BufferArchive->Num();
	int32 BytesSent = -1;

	// Try to send, log errors but avoid spaming the Log
	static bool bErrorEverLogged = false;
	if (Socket->SendTo(BufferArchive->GetData(), BufferArchive->Num(), BytesSent, *DestinationInternetAddr))
	{
		INC_DWORD_STAT(STAT_ArtNetPackagesSent);
	}
	else
	{
		if (!bErrorEverLogged)
		{
			ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
			TEnumAsByte<ESocketErrors> RecvFromError = SocketSubsystem->GetLastErrorCode();

			UE_LOGF(LogDMXProtocol, Error, "Socket subsystem failed sending DMX to %ls with Error Code %d", *DestinationInternetAddr->ToString(false), RecvFromError.GetValue());

			bErrorEverLogged = true;
		}
	}

	if (BytesSent != SendDataSize)
	{
		if (!bErrorEverLogged)
		{
			UE_LOGF(LogDMXProtocol, Warning, "Incomplete DMX Packet sent to %ls", *DestinationInternetAddr->ToString(false));
			bErrorEverLogged = true;
		}
	}
}

TSharedRef<FInternetAddr> FDMXProtocolArtNetSender::CreateBroadcastInternetAddr(int32 Port)
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);

	TSharedRef<FInternetAddr> InternetAddr = SocketSubsystem->CreateInternetAddr();
	InternetAddr->SetBroadcastAddress();
	InternetAddr->SetPort(Port);

	check(InternetAddr->IsValid());

	return InternetAddr;
}
