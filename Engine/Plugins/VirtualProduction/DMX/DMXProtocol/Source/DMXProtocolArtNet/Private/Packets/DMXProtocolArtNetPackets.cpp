// Copyright Epic Games, Inc. All Rights Reserved.

#include "Packets/DMXProtocolArtNetPackets.h"
#include "DMXProtocolTypes.h"
#include "DMXProtocolMacros.h"

#include "Serialization/BufferArchive.h"

REGISTER_DMX_ARCHIVE(FDMXProtocolArtNetDMXPacket);
REGISTER_DMX_ARCHIVE(FDMXProtocolArtNetPollPacket);
REGISTER_DMX_ARCHIVE(FDMXProtocolArtNetPacketReply);
REGISTER_DMX_ARCHIVE(FDMXProtocolArtNetTodRequest);
REGISTER_DMX_ARCHIVE(FDMXProtocolArtNetTodData);
REGISTER_DMX_ARCHIVE(FDMXProtocolArtNetTodControl);
REGISTER_DMX_ARCHIVE(FDMXProtocolArtNetRDM);

TSharedPtr<FBufferArchive> FDMXProtocolArtNetDMXPacket::Pack(const uint16 NumProperties)
{
	TSharedPtr<FBufferArchive> Writer = MakeShared<FBufferArchive>();
	*Writer << *this;

	return Writer;
}

void FDMXProtocolArtNetDMXPacket::Serialize(FArchive & Ar)
{
	Ar.Serialize((void*)ID, ARTNET_STRING_SIZE);
	Ar << OpCode;
	Ar << VerH;
	Ar << Ver;
	Ar << Sequence;
	Ar << Physical;
	Ar << Universe;
	Ar << LengthHi;
	Ar << Length;
	Ar.Serialize((void*)Data, ARTNET_DMX_LENGTH);
}


TSharedPtr<FBufferArchive> FDMXProtocolArtNetPollPacket::Pack(const uint16 NumProperties)
{
	TSharedPtr<FBufferArchive> Writer = MakeShared<FBufferArchive>();
	*Writer << *this;

	return Writer;
}

void FDMXProtocolArtNetPollPacket::Serialize(FArchive & Ar)
{
	Ar.Serialize((void*)ID, ARTNET_STRING_SIZE);
	Ar << OpCode;
	Ar << VerH;
	Ar << Ver;
	Ar << TalkToMe;
	Ar << Priority;
}

void FDMXProtocolArtNetPacketReply::Serialize(FArchive & Ar)
{
	Ar.Serialize((void*)ID, ARTNET_STRING_SIZE);
	Ar << OpCode;
	Ar.Serialize((void*)Ip, sizeof(Ip));
	Ar << Port;
	Ar << Version;
	Ar << NetAddress;
	Ar << SubnetAddress;
	Ar << Oem;
	Ar << Ubea;
	Ar << Status1;
	Ar << EstaId;
	Ar.Serialize((void*)ShortName, ARTNET_SHORT_NAME_LENGTH);
	Ar.Serialize((void*)LongName, ARTNET_LONG_NAME_LENGTH);
	Ar.Serialize((void*)NodeReport, ARTNET_REPORT_LENGTH);
	Ar.Serialize((void*)NumberPorts, sizeof(NumberPorts));
	Ar.Serialize((void*)PortTypes, ARTNET_MAX_PORTS);
	Ar.Serialize((void*)GoodInput, ARTNET_MAX_PORTS);
	Ar.Serialize((void*)GoodOutput, ARTNET_MAX_PORTS);
	Ar.Serialize((void*)SwIn, ARTNET_MAX_PORTS);
	Ar.Serialize((void*)SwOut, ARTNET_MAX_PORTS);
	Ar << SwVideo;
	Ar << SwMacro;
	Ar << SwRemote;
	Ar << Spare1;
	Ar << Spare2;
	Ar << Spare3;
	Ar << Style;
	Ar.Serialize((void*)Mac, sizeof(Mac));
	Ar.Serialize((void*)BindIp, sizeof(BindIp));
	Ar << BindIndex;
	Ar << Status2;
	Ar.Serialize((void*)Filler, sizeof(Filler));
}

TSharedPtr<FBufferArchive> FDMXProtocolArtNetTodRequest::Pack(const uint16 NumProperties)
{
	TSharedPtr<FBufferArchive> Writer = MakeShared<FBufferArchive>();
	*Writer << *this;

	return Writer;
}

void FDMXProtocolArtNetTodRequest::Serialize(FArchive & Ar)
{
	Ar.Serialize((void*)ID, ARTNET_STRING_SIZE);
	Ar << OpCode;
	Ar << VerH;
	Ar << Ver;
	Ar << Filler1;
	Ar << Filler2;
	Ar << Spare1;
	Ar << Spare2;
	Ar << Spare3;
	Ar << Spare4;
	Ar << Spare5;
	Ar << Spare6;
	Ar << Spare7;
	Ar << Net;
	Ar << Command;
	Ar << AdCount;
	Ar.Serialize((void*)Address, ARTNET_MAX_RDM_ADCOUNT);
}

TSharedPtr<FBufferArchive> FDMXProtocolArtNetTodData::Pack(const uint16 NumProperties)
{
	TSharedPtr<FBufferArchive> Writer = MakeShared<FBufferArchive>();
	*Writer << *this;

	return Writer;
}

void FDMXProtocolArtNetTodData::Serialize(FArchive & Ar)
{
	Ar.Serialize((void*)ID, ARTNET_STRING_SIZE);
	Ar << OpCode;
	Ar << VerH;
	Ar << Ver;
	Ar << RDMVer;
	Ar << Port;
	Ar << Spare1;
	Ar << Spare2;
	Ar << Spare3;
	Ar << Spare4;
	Ar << Spare5;
	Ar << Spare6;
	Ar << Spare7;
	Ar << Net;
	Ar << CmdRes;
	Ar << Address;
	Ar << UidTotalHi;
	Ar << UidTotal;
	Ar << BlockCount;
	Ar << UidCount;
	Ar.Serialize((void*)Tod, sizeof(Tod));
}

TSharedPtr<FBufferArchive> FDMXProtocolArtNetTodControl::Pack(const uint16 NumProperties)
{
	TSharedPtr<FBufferArchive> Writer = MakeShared<FBufferArchive>();
	*Writer << *this;

	return Writer;
}

void FDMXProtocolArtNetTodControl::Serialize(FArchive & Ar)
{
	Ar.Serialize((void*)ID, ARTNET_STRING_SIZE);
	Ar << OpCode;
	Ar << VerH;
	Ar << Ver;
	Ar << Filler1;
	Ar << Filler2;
	Ar << Spare1;
	Ar << Spare2;
	Ar << Spare3;
	Ar << Spare4;
	Ar << Spare5;
	Ar << Spare6;
	Ar << Spare7;
	Ar << Net;
	Ar << Cmd;
	Ar << Address;
}

TSharedPtr<FBufferArchive> FDMXProtocolArtNetRDM::Pack(const uint16 NumProperties)
{
	TSharedPtr<FBufferArchive> Writer = MakeShared<FBufferArchive>();
	*Writer << *this;

	return Writer;
}

void FDMXProtocolArtNetRDM::Serialize(FArchive & Ar)
{
	Ar.Serialize((void*)ID, ARTNET_STRING_SIZE);
	Ar << OpCode;
	Ar << VerH;
	Ar << Ver;
	Ar << RdmVer;
	Ar << Filler2;
	Ar << Spare1;
	Ar << Spare2;
	Ar << Spare3;
	Ar << Spare4;
	Ar << Spare5;
	Ar << Spare6;
	Ar << Spare7;
	Ar << Net;
	Ar << Cmd;
	Ar << Address;
	Ar.Serialize((void*)Data, ARTNET_MAX_RDM_DATA);
}

