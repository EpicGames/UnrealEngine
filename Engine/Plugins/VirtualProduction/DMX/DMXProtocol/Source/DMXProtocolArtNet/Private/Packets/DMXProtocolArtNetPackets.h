// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IDMXProtocolPacket.h"
#include "DMXProtocolArtNetConstants.h"

struct FDMXProtocolArtNetPacket
	: public IDMXProtocolPacket
{
public:
	uint8 ID[ARTNET_STRING_SIZE] = DMX_PROTOCOLNAME_ARTNET;
};

struct FDMXProtocolArtNetDMXPacket
	: public FDMXProtocolArtNetPacket
{
public:
	//~ Begin IDMXProtocolPacket implementation
	virtual TSharedPtr<FBufferArchive> Pack(const uint16 NumProperties) override;
	//~ End IDMXProtocolPort implementation

	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FDMXProtocolArtNetDMXPacket& Packet);

public:
	uint16 OpCode = ARTNET_DMX;

	uint8 VerH = 0;
	uint8 Ver = ARTNET_VERSION;
	uint8 Sequence = 0;
	uint8 Physical = 0;
	uint16 Universe = 0;
	uint8 LengthHi = DMX_SHORT_GET_HIGH_BIT(ARTNET_DMX_LENGTH);
	uint8 Length = DMX_SHORT_GET_LOW_BYTE(ARTNET_DMX_LENGTH);
	uint8 Data[ARTNET_DMX_LENGTH] = { 0 };
};

struct FDMXProtocolArtNetPollPacket
	: public FDMXProtocolArtNetPacket
{
public:
	//~ Begin IDMXProtocolPacket implementation
	virtual TSharedPtr<FBufferArchive> Pack(const uint16 NumProperties) override;
	//~ End IDMXProtocolPort implementation

	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FDMXProtocolArtNetPollPacket& Packet);
	
public:
	uint16 OpCode = ARTNET_POLL;
	uint8 VerH = 0;
	uint8 Ver = ARTNET_VERSION;
	uint8 TalkToMe = ARTNET_TTM_DEFAULT;
	uint8 Priority = 0;
};

struct FDMXProtocolArtNetPacketReply
	: public FDMXProtocolArtNetPacket
{
public:
	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FDMXProtocolArtNetPacketReply& Packet);

public:
	uint16 OpCode = 0;

	uint8 Ip[4] = { 0 };
	uint16 Port = 0;
	uint16 Version = 0;
	uint8 NetAddress = 0;
	uint8 SubnetAddress = 0;
	uint16 Oem = 0;
	uint8 Ubea = 0;
	uint8 Status1 = 0;
	uint16 EstaId = 0;
	uint8 ShortName[ARTNET_SHORT_NAME_LENGTH] = { 0 };
	uint8 LongName[ARTNET_LONG_NAME_LENGTH] = { 0 };
	uint8 NodeReport[ARTNET_REPORT_LENGTH] = { 0 };
	uint8 NumberPorts[2] = { 0 };
	uint8 PortTypes[ARTNET_MAX_PORTS] = { 0 };
	uint8 GoodInput[ARTNET_MAX_PORTS] = { 0 };
	uint8 GoodOutput[ARTNET_MAX_PORTS] = { 0 };
	uint8 SwIn[ARTNET_MAX_PORTS] = { 0 };
	uint8 SwOut[ARTNET_MAX_PORTS] = { 0 };
	uint8 SwVideo = 0;
	uint8 SwMacro = 0;
	uint8 SwRemote = 0;
	uint8 Spare1 = 0;
	uint8 Spare2 = 0;
	uint8 Spare3 = 0;
	uint8 Style = 0;
	uint8 Mac[6] = { 0 };
	uint8 BindIp[4] = { 0 };
	uint8 BindIndex = 0;
	uint8 Status2 = 0;
	uint8 Filler[26] = { 0 };
};

struct FDMXProtocolArtNetTodRequest
	: public FDMXProtocolArtNetPacket
{
public:
	//~ Begin IDMXProtocolPacket implementation
	virtual TSharedPtr<FBufferArchive> Pack(uint16) override;
	//~ End IDMXProtocolPort implementation

	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FDMXProtocolArtNetTodRequest& Packet);

public:
	uint16 OpCode = ARTNET_TODREQUEST;

	uint8 VerH = 0;
	uint8 Ver = ARTNET_VERSION;
	uint8 Filler1 = 0;
	uint8 Filler2 = 0;
	uint8 Spare1 = 0;
	uint8 Spare2 = 0;
	uint8 Spare3 = 0;
	uint8 Spare4 = 0;
	uint8 Spare5 = 0;
	uint8 Spare6 = 0;
	uint8 Spare7 = 0;
	uint8 Net = 0;
	uint8 Command = ARTNET_TOD_FULL;
	uint8 AdCount = 0;
	uint8 Address[ARTNET_MAX_RDM_ADCOUNT] = { 0 };
};

struct FDMXProtocolArtNetTodData
	: public FDMXProtocolArtNetPacket
{
public:
	//~ Begin IDMXProtocolPacket implementation
	virtual TSharedPtr<FBufferArchive> Pack(const uint16 NumProperties) override;
	//~ End IDMXProtocolPort implementation

	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FDMXProtocolArtNetTodData& Packet);

public:
	uint16 OpCode = ARTNET_TODDATA;

	uint8 VerH = 0;
	uint8 Ver = ARTNET_VERSION;
	uint8 RDMVer = ARTNET_RDM_VERSION;
	uint8 Port = 0;
	uint8 Spare1 = 0;
	uint8 Spare2 = 0;
	uint8 Spare3 = 0;
	uint8 Spare4 = 0;
	uint8 Spare5 = 0;
	uint8 Spare6 = 0;
	uint8 Spare7 = 0;
	uint8 Net = 0;
	uint8 CmdRes = ARTNET_TOD_FULL;
	uint8 Address = 0;
	uint8 UidTotalHi = 0;
	uint8 UidTotal = 0;
	uint8 BlockCount = 0;
	uint8 UidCount = 0;
	uint8 Tod[ARTNET_MAX_UID_COUNT][ARTNET_RDM_UID_WIDTH] = { { 0 } };
};

/**
 * The ArtTodControl packet is used to send RDM control parameters over Art-Net. The
 * response is ArtTodData.
 */
struct FDMXProtocolArtNetTodControl
	: public FDMXProtocolArtNetPacket
{
public:
	//~ Begin IDMXProtocolPacket implementation
	virtual TSharedPtr<FBufferArchive> Pack(const uint16 NumProperties) override;
	//~ End IDMXProtocolPort implementation

	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FDMXProtocolArtNetTodControl& Packet);

public:
	uint16 OpCode = ARTNET_TODCONTROL;

	uint8 VerH = 0;
	uint8 Ver = ARTNET_VERSION;
	uint8 Filler1 = 0;
	uint8 Filler2 = 0;
	uint8 Spare1 = 0;
	uint8 Spare2 = 0;
	uint8 Spare3 = 0;
	uint8 Spare4 = 0;
	uint8 Spare5 = 0;
	uint8 Spare6 = 0;
	uint8 Spare7 = 0;
	uint8 Net = 0;
	uint8 Cmd = 0;
	uint8 Address = 0;
};

struct FDMXProtocolArtNetRDM
	: public FDMXProtocolArtNetPacket
{
public:
	//~ Begin IDMXProtocolPacket implementation
	virtual TSharedPtr<FBufferArchive> Pack(const uint16 NumProperties) override;
	//~ End IDMXProtocolPort implementation

	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FDMXProtocolArtNetRDM& Packet);

public:
	uint16 OpCode = ARTNET_RDM;

	uint8 VerH = 0;
	uint8 Ver = ARTNET_VERSION;
	uint8 RdmVer = 0;
	uint8 Filler2 = 0;
	uint8 Spare1 = 0;
	uint8 Spare2 = 0;
	uint8 Spare3 = 0;
	uint8 Spare4 = 0;
	uint8 Spare5 = 0;
	uint8 Spare6 = 0;
	uint8 Spare7 = 0;
	uint8 Net = 0;
	uint8 Cmd = ARTNET_RDM_PROCESS_PACKET;
	uint8 Address = 0;
	uint8 Data[ARTNET_MAX_RDM_DATA] = { 0 };
};