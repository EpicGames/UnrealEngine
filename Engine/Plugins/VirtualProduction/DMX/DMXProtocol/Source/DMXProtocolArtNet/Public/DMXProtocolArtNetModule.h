// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IDMXProtocolFactory.h"

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "HAL/IConsoleManager.h"

struct FDMXProtocolRegistrationParams;


/**
 */
class FDMXProtocolFactoryArtNet : public IDMXProtocolFactory
{
public:
	virtual IDMXProtocolPtr CreateProtocol(const FName& ProtocolName) override;
};


/**
 */
class DMXPROTOCOLARTNET_API FDMXProtocolArtNetModule 
	: public IModuleInterface
{
private:

	/** Class responsible for creating instance(s) of the protocol */
	TUniquePtr<FDMXProtocolFactoryArtNet> FactoryArtNet;

public:
	UE_DEPRECATED(4.27, "Use DMX_PROTOCOLNAME_ARTNET instead")
	static FName const NAME_Artnet;

public:
	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface implementation

	/** Get the instance of this module. */
	static FDMXProtocolArtNetModule& Get();

private:
	/** Registers the Art-Net implementation here with the Protocol Module */
	void RegisterWithProtocolModule(TArray<FDMXProtocolRegistrationParams>& InOutProtocolRegistrationParamsArray);

	/**
	 * Sending DMX through console command
	 * Command structure is DMX.ArtNet.SendDMX [UniverseID] Channel:Value Channel:Value Channel:Value ...
	 * Example:
	 * DMX.ArtNet.SendDMX 17 10:6 11:7 12:8 13:9
	 * It will send the DMX to Universe 17. It could be any value from 0 to 32767
	 * And it update the channels values for channel 10, 11, 12, 13 with values 6, 7, 8, 9
	 * Channel could be any value from 0 to 511
	 * Value could be any value from 0 to 255
	 */
	static void SendDMXCommandHandler(const TArray<FString>& Args);

	/**
	 * Reset DMX through console command
	 * Command structure is DMX.ArtNet.ResetDMXSend [UniverseID]
	 * Example:
	 * DMX.ArtNet.ResetDMXSend 7
	 */
	static void ResetDMXSendUniverseHandler(const TArray<FString>& Args);

private:
	/** Command for sending DMX through the console */
	static FAutoConsoleCommand SendDMXCommand;

	/** Command for reset DMX universe through the console */
	static FAutoConsoleCommand ResetDMXSendUniverseCommand;
};
