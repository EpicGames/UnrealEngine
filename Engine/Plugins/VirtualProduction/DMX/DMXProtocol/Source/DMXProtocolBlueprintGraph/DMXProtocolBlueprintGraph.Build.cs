// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXProtocolBlueprintGraph : ModuleRules
{
	public DMXProtocolBlueprintGraph(ReadOnlyTargetRules Target) : base(Target)
    {
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "DMXProtocol",
            }
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
                "ApplicationCore",
                "AssetRegistry",
                "AssetTools",
                "CoreUObject",
				"Kismet",
                "KismetCompiler",
                "UnrealEd",
                
                "PropertyEditor",
                
                "KismetWidgets",
                "Engine",
                "Slate",
                "SlateCore",
                "InputCore",
                "Json",
                "Projects",
                "BlueprintGraph",
                "GraphEditor",
				"DMXProtocolEditor"
            }
		);
	}
}