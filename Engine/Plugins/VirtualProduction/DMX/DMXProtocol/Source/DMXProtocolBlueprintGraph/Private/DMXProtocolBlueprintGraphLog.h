// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDMXProtocolBlueprintGraph, Log, All);

#define UE_LOG_DMXPROTOCOLBLUEPRINTGRAPH(Verbosity, Format, ...) \
{ \
	UE_LOG(LogDMXProtocolBlueprintGraph, Verbosity, Format, ##__VA_ARGS__); \
}

#define UE_CLOG_DMXPROTOCOLBLUEPRINTGRAPH(Conditional, Verbosity, Format, ...) \
{ \
	UE_CLOG(Conditional, LogDMXProtocolBlueprintGraph, Verbosity, Format, ##__VA_ARGS__); \
}
