// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolBlueprintGraphModule.h"

#include "DMXProtocolGraphPanelPinFactory.h"
#include "DMXProtocolBlueprintGraphLog.h"

#include "AssetToolsModule.h"
#include "PropertyEditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "DMXProtocolBlueprintGraphModule"


void FDMXProtocolBlueprintGraphModule::StartupModule()
{
	DMXProtocolGraphPanelPinFactory = MakeShared<FDMXProtocolGraphPanelPinFactory>();

	FEdGraphUtilities::RegisterVisualPinFactory(DMXProtocolGraphPanelPinFactory);
}

void FDMXProtocolBlueprintGraphModule::ShutdownModule()
{
	FEdGraphUtilities::UnregisterVisualPinFactory(DMXProtocolGraphPanelPinFactory);

	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Unregister all classes customized by name
		for (const FName& ClassName : RegisteredClassNames)
		{
			PropertyModule.UnregisterCustomClassLayout(ClassName);
		}

		PropertyModule.NotifyCustomizationModuleChanged();
	}
}

void FDMXProtocolBlueprintGraphModule::RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate)
{
	check(ClassName != NAME_None);

	RegisteredClassNames.Add(ClassName);

	static FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.RegisterCustomClassLayout(ClassName, DetailLayoutDelegate);
}

IMPLEMENT_MODULE(FDMXProtocolBlueprintGraphModule, DMXProtocolBlueprintGraph)

#undef LOCTEXT_NAMESPACE
