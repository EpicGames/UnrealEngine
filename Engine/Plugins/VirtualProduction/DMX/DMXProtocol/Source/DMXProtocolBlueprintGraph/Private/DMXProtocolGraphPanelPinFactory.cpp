// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolGraphPanelPinFactory.h"

#include "Widgets/SDMXInputPortReferenceGraphPin.h"
#include "Widgets/SDMXOutputPortReferenceGraphPin.h"

#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "UObject/Class.h"


TSharedPtr<class SGraphPin> FDMXProtocolGraphPanelPinFactory::CreatePin(class UEdGraphPin* InPin) const
{
	if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		if (UScriptStruct* PinStructType = Cast<UScriptStruct>(InPin->PinType.PinSubCategoryObject.Get()))
		{
			if (PinStructType->IsChildOf(FDMXInputPortReference::StaticStruct()))
			{
				return SNew(SDMXInputPortReferenceGraphPin, InPin);
			}
			else if (PinStructType->IsChildOf(FDMXOutputPortReference::StaticStruct()))
			{
				return SNew(SDMXOutputPortReferenceGraphPin, InPin);
			}
		}
	}

	return FGraphPanelPinFactory::CreatePin(InPin);
}