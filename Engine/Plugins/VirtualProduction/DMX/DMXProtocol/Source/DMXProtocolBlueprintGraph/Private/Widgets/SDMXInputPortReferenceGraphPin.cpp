// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXInputPortReferenceGraphPin.h"

#include "DMXProtocolSettings.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXInputPortReference.h"
#include "Widgets/SDMXPortSelector.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SDMXInputPortReferenceGraphPin"

void SDMXInputPortReferenceGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget> SDMXInputPortReferenceGraphPin::GetDefaultValueWidget()
{
	FDMXInputPortReference InitiallySelectedPortReference = GetPinValue();

	PortSelector = SNew(SDMXPortSelector)
		.Mode(EDMXPortSelectorMode::SelectFromAvailableInputs)
		.InitialSelection(InitiallySelectedPortReference.GetPortGuid())
		.OnPortSelected(this, &SDMXInputPortReferenceGraphPin::OnPortSelected)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility);
	
	return PortSelector.ToSharedRef();
}

FDMXInputPortReference SDMXInputPortReferenceGraphPin::GetPinValue() const
{
	FDMXInputPortReference PortReference;

	const FString EntityRefStr = GraphPinObj->GetDefaultAsString();
	if (!EntityRefStr.IsEmpty())
	{
		FDMXInputPortReference::StaticStruct()->ImportText(*EntityRefStr, &PortReference, nullptr, EPropertyPortFlags::PPF_None, GLog, FDMXInputPortReference::StaticStruct()->GetName());
	}

	return PortReference;
}

void SDMXInputPortReferenceGraphPin::SetPinValue(const FDMXInputPortReference& InputPortReference, bool bMarkAsModified) const
{
	if (GraphPinObj->IsPendingKill())
	{
		return;
	}

	FString ValueString;
	FDMXInputPortReference::StaticStruct()->ExportText(ValueString, &InputPortReference, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);

	GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ValueString, bMarkAsModified);
}

void SDMXInputPortReferenceGraphPin::OnPortSelected() const
{
	if (PortSelector.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeObjectPinValue", "Select DMX Port"));
		GraphPinObj->Modify();

		const FGuid& PortGuid = [this]() {
			if (FDMXInputPortSharedPtr SelectedInputPort = PortSelector->GetSelectedInputPort())
			{
				return SelectedInputPort->GetPortGuid();
			}
			else if (FDMXOutputPortSharedPtr SelectedOutputPort = PortSelector->GetSelectedOutputPort())
			{
				return SelectedOutputPort->GetPortGuid();
			}

			return FGuid::NewGuid();
		}();


		FDMXInputPortReference PortReference(PortGuid, true);

		constexpr bool bMarkAsModified = true;
		SetPinValue(PortReference, bMarkAsModified);
	}
}

#undef LOCTEXT_NAMESPACE
