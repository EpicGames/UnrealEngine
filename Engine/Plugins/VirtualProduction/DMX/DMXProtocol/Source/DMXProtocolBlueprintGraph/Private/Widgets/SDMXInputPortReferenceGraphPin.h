// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/DMXInputPortReference.h"

#include "CoreMinimal.h"
#include "SGraphPin.h"

class SDMXPortSelector;


class DMXPROTOCOLBLUEPRINTGRAPH_API SDMXInputPortReferenceGraphPin
	: public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SDMXInputPortReferenceGraphPin)
	{}
	
	SLATE_END_ARGS()

	/**  Slate widget construction method */
	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

private:
	void OnPortSelected() const;

	FDMXInputPortReference GetPinValue() const;

	void SetPinValue(const FDMXInputPortReference& InputPortReference, bool bMarkAsModified) const;

	TSharedPtr<SDMXPortSelector> PortSelector;
};
