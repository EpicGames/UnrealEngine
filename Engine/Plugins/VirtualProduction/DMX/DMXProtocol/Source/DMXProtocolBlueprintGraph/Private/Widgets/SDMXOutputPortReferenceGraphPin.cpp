// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXOutputPortReferenceGraphPin.h"

#include "DMXProtocolSettings.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXOutputPortReference.h"
#include "Widgets/SDMXPortSelector.h"

#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SDMXOutputPortReferenceGraphPin"

void SDMXOutputPortReferenceGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget> SDMXOutputPortReferenceGraphPin::GetDefaultValueWidget()
{
	FDMXOutputPortReference InitiallySelectedPortReference = GetPinValue();

	PortSelector = SNew(SDMXPortSelector)
		.Mode(EDMXPortSelectorMode::SelectFromAvailableOutputs)
		.InitialSelection(InitiallySelectedPortReference.GetPortGuid())
		.OnPortSelected(this, &SDMXOutputPortReferenceGraphPin::OnPortSelected)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility);
	
	return PortSelector.ToSharedRef();
}

FDMXOutputPortReference SDMXOutputPortReferenceGraphPin::GetPinValue() const
{
	FDMXOutputPortReference PortReference;

	const FString EntityRefStr = GraphPinObj->GetDefaultAsString();
	if (!EntityRefStr.IsEmpty())
	{
		FDMXOutputPortReference::StaticStruct()->ImportText(*EntityRefStr, &PortReference, nullptr, EPropertyPortFlags::PPF_None, GLog, FDMXOutputPortReference::StaticStruct()->GetName());
	}

	return PortReference;
}

void SDMXOutputPortReferenceGraphPin::SetPinValue(const FDMXOutputPortReference& OutputPortReference, bool bMarkAsModified) const
{
	if (GraphPinObj->IsPendingKill())
	{
		return;
	}

	FString ValueString;
	FDMXOutputPortReference::StaticStruct()->ExportText(ValueString, &OutputPortReference, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);

	GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ValueString, bMarkAsModified);
}

void SDMXOutputPortReferenceGraphPin::OnPortSelected() const
{
	const FScopedTransaction Transaction(LOCTEXT("ChangeObjectPinValue", "Select DMX Port"));
	GraphPinObj->Modify();

	const FGuid& PortGuid = [this]() {
		if (FDMXOutputPortSharedPtr SelectedOutputPort = PortSelector->GetSelectedOutputPort())
		{
			return SelectedOutputPort->GetPortGuid();
		}

		return FGuid::NewGuid();
	}();

	const FDMXOutputPortReference PortReference(PortGuid, true);
	SetPinValue(PortReference);
}

#undef LOCTEXT_NAMESPACE
