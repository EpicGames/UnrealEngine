// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/DMXOutputPortReference.h"

#include "CoreMinimal.h"
#include "SGraphPin.h"

class SDMXPortSelector;


class DMXPROTOCOLBLUEPRINTGRAPH_API SDMXOutputPortReferenceGraphPin
	: public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SDMXOutputPortReferenceGraphPin)
	{}

	SLATE_END_ARGS()

		/**  Slate widget construction method */
		void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

private:
	void OnPortSelected() const;

	FDMXOutputPortReference GetPinValue() const;

	void SetPinValue(const FDMXOutputPortReference& OutputPortReference, bool bMarkAsModified = true) const;

	TSharedPtr<SDMXPortSelector> PortSelector;
};
