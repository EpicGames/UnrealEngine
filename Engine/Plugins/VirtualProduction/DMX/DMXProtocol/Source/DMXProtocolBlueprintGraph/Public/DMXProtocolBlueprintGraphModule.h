// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"

class FDMXProtocolGraphPanelPinFactory;


/**  The public interface to this module */
class FDMXProtocolBlueprintGraphModule 
	: public IModuleInterface
{

public:

	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

	//~ End IModuleInterface implementation

	static FDMXProtocolBlueprintGraphModule& Get()
	{
		return FModuleManager::LoadModuleChecked< FDMXProtocolBlueprintGraphModule >("DMXProtocolBlueprintGraphModule");
	}

	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "IDMXProtocolBlueprintGraphModule" );
	}

private:
	/**
	 * Registers a custom class
	 *
	 * @param ClassName				The class name to register for property customization
	 * @param DetailLayoutDelegate	The delegate to call to get the custom detail layout instance
	 */
	void RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate);

private:

	/** FDMXProtocolName and Custom nodes Graph Pin customizations */
	TSharedPtr<FDMXProtocolGraphPanelPinFactory> DMXProtocolGraphPanelPinFactory;

	/** List of registered class that we must unregister when the module shuts down */
	TSet< FName > RegisteredClassNames;
};

