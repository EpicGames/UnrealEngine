// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXProtocolEditor : ModuleRules
{
	public DMXProtocolEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"DMXProtocol",
				"Networking",
				"Sockets",
				"PropertyEditor"
			}
		);
				
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				
				"Engine",
				"InputCore",
				"Projects",
				"Slate",
				"SlateCore",
				"TimeManagement",
				"UnrealEd"
			}
		);

	}
}
