// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolEditorModule.h"

#include "DMXProtocolSettings.h"
#include "IO/DMXInputPortReference.h"
#include "IO/DMXOutputPortReference.h"
#include "DetailsCustomizations/DMXInputPortConfigCustomization.h"
#include "DetailsCustomizations/DMXOutputPortConfigCustomization.h"
#include "DetailsCustomizations/DMXOutputPortDestinationAddressCustomization.h"
#include "DetailsCustomizations/DMXInputPortReferenceCustomization.h"
#include "DetailsCustomizations/DMXOutputPortReferenceCustomization.h"

#include "PropertyEditorModule.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"



#define LOCTEXT_NAMESPACE "DMXProtocolEditorModule"

void FDMXProtocolEditorModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FDMXProtocolEditorModule::RegisterDetailsCustomizations);
}

void FDMXProtocolEditorModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	UnregisterDetailsCustomizations();
}

FDMXProtocolEditorModule& FDMXProtocolEditorModule::Get()
{
	return FModuleManager::GetModuleChecked<FDMXProtocolEditorModule>("DMXProtocolEditor");
}

void FDMXProtocolEditorModule::RegisterDetailsCustomizations()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomPropertyTypeLayout(FDMXInputPortConfig::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXInputPortConfigCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FDMXOutputPortConfig::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXOutputPortConfigCustomization::MakeInstance));
	
	PropertyModule.RegisterCustomPropertyTypeLayout(FDMXOutputPortDestinationAddress::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXOutputPortDestinationAddressCustomization::MakeInstance));

	PropertyModule.RegisterCustomPropertyTypeLayout(FDMXInputPortReference::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXInputPortReferenceCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FDMXOutputPortReference::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXOutputPortReferenceCustomization::MakeInstance));

	PropertyModule.NotifyCustomizationModuleChanged();
}

void FDMXProtocolEditorModule::UnregisterDetailsCustomizations()
{
	if (UObjectInitialized() && FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");

		PropertyModule->UnregisterCustomPropertyTypeLayout(FDMXInputPortConfig::StaticStruct()->GetFName());
		PropertyModule->UnregisterCustomPropertyTypeLayout(FDMXOutputPortConfig::StaticStruct()->GetFName());

		PropertyModule->UnregisterCustomPropertyTypeLayout(FDMXOutputPortDestinationAddress::StaticStruct()->GetFName());

		PropertyModule->UnregisterCustomPropertyTypeLayout(FDMXInputPortReference::StaticStruct()->GetFName());
		PropertyModule->UnregisterCustomPropertyTypeLayout(FDMXOutputPortReference::StaticStruct()->GetFName());
	}
}

IMPLEMENT_MODULE(FDMXProtocolEditorModule, DMXProtocolEditor);

#undef LOCTEXT_NAMESPACE
