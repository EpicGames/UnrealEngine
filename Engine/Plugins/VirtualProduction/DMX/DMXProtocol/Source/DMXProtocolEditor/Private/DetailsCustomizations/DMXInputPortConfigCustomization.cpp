// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXInputPortConfigCustomization.h"

#include "DetailWidgetRow.h"
#include "DMXProtocolModule.h"
#include "DMXProtocolSettings.h"
#include "DMXProtocolTypes.h"
#include "DMXProtocolUtils.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h" 
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXInputPortConfig.h"
#include "IO/DMXPortManager.h"
#include "IPropertyUtilities.h"
#include "Misc/Guid.h" 
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SDMXCommunicationTypeComboBox.h"
#include "Widgets/SDMXIPAddressEditWidget.h"
#include "Widgets/SDMXProtocolNameComboBox.h"
#include "Widgets/Text/STextBlock.h" 


#define LOCTEXT_NAMESPACE "DMXInputPortConfigCustomization"

TSharedRef<IPropertyTypeCustomization> FDMXInputPortConfigCustomization::MakeInstance()
{
	return MakeShared<FDMXInputPortConfigCustomization>();
}

void FDMXInputPortConfigCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

void FDMXInputPortConfigCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!StructPropertyHandle->IsExpanded())
	{
		StructPropertyHandle->SetExpanded(true);
	}

	// Retrieve structure's child properties
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	TMap<FName, TSharedPtr< IPropertyHandle>> PropertyHandles;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	// Cache customized properties
	ProtocolNameHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetProtocolNamePropertyNameChecked());
	CommunicationTypeHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetCommunicationTypePropertyNameChecked());
	AutoCompleteDeviceAddressEnabledHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetAutoCompleteDeviceAddressEnabledPropertyNameChecked());
	AutoCompleteDeviceAddressHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetAutoCompleteDeviceAddressPropertyNameChecked());
	DeviceAddressHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetDeviceAddressPropertyNameChecked());
	LocalUniverseStartHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetLocalUniverseStartPropertyNameChecked());
	ExternUniverseStartHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetExternUniverseStartPropertyNameChecked());
	IsExternUniverseStartEditableHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetIsExternUnivereStartEditablePropertyNameChecked());
	PortGuidHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetPortGuidPropertyNameChecked());

	// Hande property changes
	AutoCompleteDeviceAddressHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXInputPortConfigCustomization::UpdateAutoCompleteDeviceAddressTextBox));
	LocalUniverseStartHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXInputPortConfigCustomization::OnLocalUniverseStartChanged));
	IsExternUniverseStartEditableHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXInputPortConfigCustomization::OnIsExternUniverseStartEditableChanged));

	// Ports always need a valid Guid (cannot be blueprinted)
	if (!GetPortGuid().IsValid())
	{
		ChildBuilder.AddCustomRow(LOCTEXT("InvalidPortConfigSearchString", "Invalid"))
			.WholeRowContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("InvalidPortConfigText", "Invalid Port Guid. Cannot utilize this port."))
			];

		return;
	}

	// Add customized properties
	for (auto Iter(PropertyHandles.CreateConstIterator()); Iter; ++Iter)
	{
		// Don't add the PortGuid property
		if (Iter.Value() == PortGuidHandle)
		{
			continue;
		}

		// Don't add the bAutoCompeteDeviceAddressEnabled property
		// Don't add the bIsExternUniverseStartEditable property
		if (Iter.Value() == AutoCompleteDeviceAddressEnabledHandle ||
			Iter.Value() == IsExternUniverseStartEditableHandle)
		{
			continue;
		}
		
		// Add the property
		IDetailPropertyRow& PropertyRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());

		if (Iter.Value() == ProtocolNameHandle)
		{
			GenerateProtocolNameRow(PropertyRow);
		}
		else if (Iter.Value() == CommunicationTypeHandle)
		{
			GenerateCommunicationTypeRow(PropertyRow);
		}
		else if (Iter.Value() == AutoCompleteDeviceAddressHandle)
		{
			GenerateAutoCompleteDeviceAddressRow(PropertyRow);
		}
		else if (Iter.Value() == DeviceAddressHandle)
		{
			GenerateDeviceAddressRow(PropertyRow);
		}
		else if (Iter.Value() == ExternUniverseStartHandle)
		{
			GenerateExternUniverseStartRow(PropertyRow);
		}
		else if (Iter.Key() == FDMXInputPortConfig::GetPriorityStrategyPropertyNameChecked())
		{

			PropertyRow.Visibility(TAttribute<EVisibility>::CreateLambda([this]
				{
					const bool bPriorityStrategyIsVisible = GetProtocol().IsValid() && GetProtocol()->SupportsPrioritySettings();
					return bPriorityStrategyIsVisible ? 
						EVisibility::Visible : 
						EVisibility::Collapsed;
				}));
		}
		else if (Iter.Key() == FDMXInputPortConfig::GetPriorityPropertyNameChecked())
		{
			PropertyRow.Visibility(TAttribute<EVisibility>::CreateLambda([this]
				{
					const bool bPriorityIsVisible = GetProtocol().IsValid() && GetProtocol()->SupportsPrioritySettings();
					return bPriorityIsVisible ? 
						EVisibility::Visible : 
						EVisibility::Collapsed;
				}));
		}
	}
}

void FDMXInputPortConfigCustomization::GenerateProtocolNameRow(IDetailPropertyRow& PropertyRow)
{
	// Customizate the Protocol Name property to draw a combo box with all protocol names
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
	
	const FName InitialSelection = [this]() -> FName
	{
		if (const IDMXProtocolPtr& Protocol = GetProtocol())
		{
			return Protocol->GetProtocolName();
		}

		return NAME_None;
	}();

	PropertyRow.CustomWidget()
		.NameContent()
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		[
			SAssignNew(ProtocolNameComboBox, SDMXProtocolNameComboBox)
			.InitiallySelectedProtocolName(InitialSelection)
			.OnProtocolNameSelected(this, &FDMXInputPortConfigCustomization::OnProtocolNameSelected)
		];

}

void FDMXInputPortConfigCustomization::GenerateCommunicationTypeRow(IDetailPropertyRow& PropertyRow)
{
	// Customizate the Communication Type property depending on the port's supported communication modes
	const TAttribute<EVisibility> CommunicationTypeVisibilityAttribute = TAttribute<EVisibility>::CreateLambda([this]
		{
			if (CommunicationTypeComboBox.IsValid() && GetProtocol().IsValid())
			{
				return CommunicationTypeComboBox->GetVisibility();
			}
			return EVisibility::Collapsed;
		});

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	PropertyRow.CustomWidget()
		.Visibility(CommunicationTypeVisibilityAttribute)
		.NameContent()
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		[
			SAssignNew(CommunicationTypeComboBox, SDMXCommunicationTypeComboBox)
			.CommunicationTypes(GetSupportedCommunicationTypes())
			.InitialCommunicationType(GetCommunicationType())
			.OnCommunicationTypeSelected(this, &FDMXInputPortConfigCustomization::OnCommunicationTypeSelected)
		];
}

void FDMXInputPortConfigCustomization::GenerateDeviceAddressRow(IDetailPropertyRow& PropertyRow)
{
	// Customizate the IPAddress property to show a combo box with available IP addresses
	const FString InitialValue = GetIPAddress();

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	// Override the value widget so it either shows 
	// - The IP Address Edit widget when auto complete is disabled
	// - The auto completed IP Address when auto complete is enabled
	ValueWidget =
		SNew(SHorizontalBox)
		
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(IPAddressEditWidget, SDMXIPAddressEditWidget)
			.Visibility_Lambda([this]()
				{
					return IsAutoCompleteDeviceAddressEnabled() ?
						EVisibility::Collapsed :
						EVisibility::Visible;
				})
			.InitialValue(InitialValue)
			.bShowLocalNICComboBox(true)
			.OnIPAddressSelected(this, &FDMXInputPortConfigCustomization::OnDeviceAddressSelected)
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(AutoCompletedDeviceAddressTextBlock, STextBlock)
			.Visibility_Lambda([this]()
				{
					return IsAutoCompleteDeviceAddressEnabled() ?
						EVisibility::Visible :
						EVisibility::Collapsed;
				})
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		];
	UpdateAutoCompleteDeviceAddressTextBox();

	PropertyRow.IsEnabled(TAttribute<bool>::CreateLambda([this]
		{
			return !IsAutoCompleteDeviceAddressEnabled();
		}));

	PropertyRow.CustomWidget()
		.NameContent()
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		[
			ValueWidget.ToSharedRef()
		];
}

void FDMXInputPortConfigCustomization::GenerateAutoCompleteDeviceAddressRow(IDetailPropertyRow& PropertyRow)
{
	// Mimic a Edit Condition Toggle - Doing it directly via Meta Specifiers is not possible. -
	// The Edit Condition Toggle property wouldn't get properly serialized when being a Config property.
	const TSharedRef<SWidget> AutoCompleteDeviceAddressEnableldPropertyValueWidget = AutoCompleteDeviceAddressEnabledHandle->CreatePropertyValueWidget();

	const TSharedRef<SWidget> AutoCompleteDeviceAddressPropertyNameWidget = AutoCompleteDeviceAddressHandle->CreatePropertyNameWidget();
	const TSharedRef<SWidget> AutoCompleteDeviceAddressPropertyValueWidget = AutoCompleteDeviceAddressHandle->CreatePropertyValueWidget();

	AutoCompleteDeviceAddressPropertyNameWidget->SetEnabled(TAttribute<bool>::CreateLambda([this]()
		{
			return IsAutoCompleteDeviceAddressEnabled();
		}));
	AutoCompleteDeviceAddressPropertyValueWidget->SetEnabled(TAttribute<bool>::CreateLambda([this]()
		{
			return IsAutoCompleteDeviceAddressEnabled();
		}));

	PropertyRow.CustomWidget()
		.NameContent()
		[
			SNew(SHorizontalBox)
	
			// Edit Condition Toggle Checkbox
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.AutoWidth()
			[
				AutoCompleteDeviceAddressEnableldPropertyValueWidget
			]

			// Property Name
			+ SHorizontalBox::Slot()
			.Padding(8.f, 0.f, 0.f, 0.f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.AutoWidth()
			[
				AutoCompleteDeviceAddressPropertyNameWidget
			]
		]
		.ValueContent()
		[
			// Property Value
			AutoCompleteDeviceAddressPropertyValueWidget
		];
}

void FDMXInputPortConfigCustomization::UpdateAutoCompleteDeviceAddressTextBox()
{
	// Set text on the AutoCompleteDeviceAddress Text Box depending on what the resulting IP is
	if (AutoCompletedDeviceAddressTextBlock.IsValid())
	{
		FString AutoCompleteDeviceAddress;
		if (AutoCompleteDeviceAddressHandle->GetValue(AutoCompleteDeviceAddress) == FPropertyAccess::Success)
		{
			FString NetworkInterfaceCardIPAddress;
			if (FDMXProtocolUtils::FindLocalNetworkInterfaceCardIPAddress(AutoCompleteDeviceAddress, NetworkInterfaceCardIPAddress))
			{
				const FText IPAddressText = FText::FromString(*NetworkInterfaceCardIPAddress);
				AutoCompletedDeviceAddressTextBlock->SetText(FText::Format(LOCTEXT("AutoCompletedIPAddressText", "{0} (auto-completed)"), IPAddressText));
			}
			else
			{
				// Fall back to the manually set one
				FString DeviceAddress;
				DeviceAddressHandle->GetValue(DeviceAddress);
				const FText NoMatchForIPAddressText = FText::Format(LOCTEXT("NoRegexMatchForDeviceAddress", "No match. Using '{0}'"), FText::FromString(DeviceAddress));
				AutoCompletedDeviceAddressTextBlock->SetText(NoMatchForIPAddressText);
			}
		}
	}
}

void FDMXInputPortConfigCustomization::GenerateExternUniverseStartRow(IDetailPropertyRow& PropertyRow)
{
	const TAttribute<bool> CanEditExternUniverseAttribute = TAttribute<bool>::CreateLambda([this]()
		{
			bool bIsExternUniverseStartEditable;
			if (IsExternUniverseStartEditableHandle->GetValue(bIsExternUniverseStartEditable) == FPropertyAccess::Success)
			{
				return bIsExternUniverseStartEditable ? true : false;
			}
			return false;
		});

	TSharedPtr<SWidget> ExternUniverseStartNameWidget;
	TSharedPtr<SWidget> ExternUniverseStartValueWidget;
	FDetailWidgetRow ExternUniverseStartRow;
	PropertyRow.GetDefaultWidgets(ExternUniverseStartNameWidget, ExternUniverseStartValueWidget, ExternUniverseStartRow);

	const FText IsExternUniverseEnabledTooltip = LOCTEXT("IsExternUniverseEnabledWarningTooltip", "Remaps the range of local Universes (Num Universes from Local Universe Start) to a different set of Universes at the Protocol level.\nFor example, for Art-Net this allows to map local Universe 1 to extern Universe 0.");

	PropertyRow.CustomWidget()
		.NameContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 4.f, 0.f)
			[
				SNew(SCheckBox)
				.ToolTipText(IsExternUniverseEnabledTooltip)
				.IsChecked_Lambda([CanEditExternUniverseAttribute]()
					{
						return CanEditExternUniverseAttribute.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
					{
						const bool bIsExternUniverseStartEditable = NewState == ECheckBoxState::Checked;
						IsExternUniverseStartEditableHandle->SetValue(bIsExternUniverseStartEditable);
					})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.IsEnabled(CanEditExternUniverseAttribute)
				[
					ExternUniverseStartNameWidget.ToSharedRef()
				]
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			.IsEnabled(CanEditExternUniverseAttribute)

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, 4.f, 0.f)
			[
				ExternUniverseStartValueWidget.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.f)
			[
				SNew(SImage)
				.Visibility_Lambda([CanEditExternUniverseAttribute]()
					{
						return CanEditExternUniverseAttribute.Get() ? EVisibility::Visible : EVisibility::Collapsed;
					})
				.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
				.ToolTipText(IsExternUniverseEnabledTooltip)
			]
		];
}

void FDMXInputPortConfigCustomization::OnLocalUniverseStartChanged()
{
	bool bCanEditExternUniverse;
	int32 LocalUniverse;
	if (IsExternUniverseStartEditableHandle->GetValue(bCanEditExternUniverse) == FPropertyAccess::Success &&
		!bCanEditExternUniverse &&
		LocalUniverseStartHandle->GetValue(LocalUniverse) == FPropertyAccess::Success)
	{
		ExternUniverseStartHandle->NotifyPreChange();
		// Change the extern universe interactive only. Rely on the local universe start change completing the transaction.
		ExternUniverseStartHandle->SetValue(LocalUniverse, EPropertyValueSetFlags::InteractiveChange);
		ExternUniverseStartHandle->NotifyPostChange(EPropertyChangeType::Interactive);
	}
}

void FDMXInputPortConfigCustomization::OnIsExternUniverseStartEditableChanged()
{
	bool bCanEditExternUniverse;
	int32 LocalUniverse;
	if (IsExternUniverseStartEditableHandle->GetValue(bCanEditExternUniverse) == FPropertyAccess::Success &&
		!bCanEditExternUniverse &&
		LocalUniverseStartHandle->GetValue(LocalUniverse) == FPropertyAccess::Success)
	{
		ExternUniverseStartHandle->NotifyPreChange();
		ExternUniverseStartHandle->SetValue(LocalUniverse);
		ExternUniverseStartHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

void FDMXInputPortConfigCustomization::OnProtocolNameSelected()
{
	FName ProtocolName = ProtocolNameComboBox->GetSelectedProtocolName();

	const FScopedTransaction Transaction(LOCTEXT("ProtocolSelected", "DMX: Selected Protocol"));
	
	ProtocolNameHandle->NotifyPreChange();
	ProtocolNameHandle->SetValue(ProtocolName);
	ProtocolNameHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	PropertyUtilities->ForceRefresh();
}

void FDMXInputPortConfigCustomization::OnCommunicationTypeSelected()
{
	EDMXCommunicationType SelectedCommunicationType = CommunicationTypeComboBox->GetSelectedCommunicationType();

	const FScopedTransaction Transaction(LOCTEXT("CommunicationTypeSelected", "DMX: Selected Communication Type"));
	
	CommunicationTypeHandle->NotifyPreChange();
	CommunicationTypeHandle->SetValue(static_cast<uint8>(SelectedCommunicationType));
	CommunicationTypeHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	PropertyUtilities->ForceRefresh();
}

void FDMXInputPortConfigCustomization::OnDeviceAddressSelected()
{
	FString SelectedIP = IPAddressEditWidget->GetSelectedIPAddress();

	const FScopedTransaction Transaction(LOCTEXT("IPAddressSelected", "DMX: Selected IP Address"));
	
	DeviceAddressHandle->NotifyPreChange();
	ensure(DeviceAddressHandle->SetValue(SelectedIP) == FPropertyAccess::Success);	
	DeviceAddressHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

IDMXProtocolPtr FDMXInputPortConfigCustomization::GetProtocol() const
{
	FName ProtocolName;
	ProtocolNameHandle->GetValue(ProtocolName);

	IDMXProtocolPtr Protocol = IDMXProtocol::Get(ProtocolName);

	return Protocol;
}

FGuid FDMXInputPortConfigCustomization::GetPortGuid() const
{
	TArray<void*> RawData;
	PortGuidHandle->AccessRawData(RawData);

	// Multiediting is not supported, may fire if this is used in a blueprint way that would support it
	if (ensureMsgf(RawData.Num() == 1, TEXT("Using port config in ways that would enable multiediting is not supported.")))
	{
		const FGuid* PortGuidPtr = reinterpret_cast<FGuid*>(RawData[0]);
		if (PortGuidPtr && PortGuidPtr->IsValid())
		{
			return *PortGuidPtr;
		}
	}

	return FGuid();
}

const TArray<EDMXCommunicationType> FDMXInputPortConfigCustomization::GetSupportedCommunicationTypes() const
{
	if (IDMXProtocolPtr Protocol = GetProtocol())
	{
		return Protocol->GetInputPortCommunicationTypes();
	}

	return TArray<EDMXCommunicationType>();
}

EDMXCommunicationType FDMXInputPortConfigCustomization::GetCommunicationType() const
{
	uint8 CommunicationType;
	ensure(CommunicationTypeHandle->GetValue(CommunicationType) == FPropertyAccess::Success);

	return static_cast<EDMXCommunicationType>(CommunicationType);
}

bool FDMXInputPortConfigCustomization::IsAutoCompleteDeviceAddressEnabled() const
{
	bool bAutoCompleteDeviceAddressEnabled;
	AutoCompleteDeviceAddressEnabledHandle->GetValue(bAutoCompleteDeviceAddressEnabled);

	return bAutoCompleteDeviceAddressEnabled;
}

FString FDMXInputPortConfigCustomization::GetIPAddress() const
{
	FString IPAddress;
	ensure(DeviceAddressHandle->GetValue(IPAddress) == FPropertyAccess::Success);

	return IPAddress;
}

#undef LOCTEXT_NAMESPACE
