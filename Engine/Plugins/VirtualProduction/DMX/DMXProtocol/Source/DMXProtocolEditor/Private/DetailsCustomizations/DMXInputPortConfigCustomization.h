// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"

enum class EDMXCommunicationType : uint8;
class SDMXCommunicationTypeComboBox;
class SDMXIPAddressEditWidget;
class SDMXProtocolNameComboBox;

class IDetailPropertyRow;
class IPropertyHandle;
class IPropertyUtilities;
class STextBlock;


/** Details customization for input and output port configs. */
class FDMXInputPortConfigCustomization
	: public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:
	// ~Begin IPropertyTypecustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// ~End IPropertyTypecustomization Interface

private:
	/** Generates the customized Protocol Name row */
	void GenerateProtocolNameRow(IDetailPropertyRow& PropertyRow);

	/** Generates the customized Communication Type row */
	void GenerateCommunicationTypeRow(IDetailPropertyRow& PropertyRow);

	/** Generates the customized Device Address row */
	void GenerateDeviceAddressRow(IDetailPropertyRow& PropertyRow);

	/** Generates the customized Auto Complete Device Address row */
	void GenerateAutoCompleteDeviceAddressRow(IDetailPropertyRow& PropertyRow);

	/** Updates the Auto Complete Device Address Text Box */
	void UpdateAutoCompleteDeviceAddressTextBox();

	/** Generates the customized Extern Universe Start row */
	void GenerateExternUniverseStartRow(IDetailPropertyRow& PropertyRow);

	/** Called when the LocalUniverseStart property changed */
	void OnLocalUniverseStartChanged();

	/** Called when the bIsExternUniverseStartEditable property changed */
	void OnIsExternUniverseStartEditableChanged();

	/** Called when a Protocol Name was selected */
	void OnProtocolNameSelected();

	/** Called when a Communication Type was selected */
	void OnCommunicationTypeSelected();

	/** Called when a local IP Address was selected */
	void OnDeviceAddressSelected();

	/** Gets the protocol, checks if it is valid */
	IDMXProtocolPtr GetProtocol() const;

	/** Helper function that gets the Guid of the edited port */
	FGuid GetPortGuid() const;

	/** Returns an array of supported communication types */
	const TArray<EDMXCommunicationType> GetSupportedCommunicationTypes() const;

	/** Gets the communication type */
	EDMXCommunicationType GetCommunicationType() const;

	/** Returns the value of the AutoCompleteDeviceAddressEnabled property */
	bool IsAutoCompleteDeviceAddressEnabled() const;

	/** Gets the IP Address */
	FString GetIPAddress() const;

	/** Property handle to the ProtocolName property */
	TSharedPtr<IPropertyHandle> ProtocolNameHandle;

	/** Property handle to the bAutoCompleteDeviceAddressEnabled property */
	TSharedPtr<IPropertyHandle> AutoCompleteDeviceAddressEnabledHandle;

	/** Property handle to the AutoCompleteDeviceAddress property */
	TSharedPtr<IPropertyHandle> AutoCompleteDeviceAddressHandle;

	/** Property handle to the IPAddress property */
	TSharedPtr<IPropertyHandle> DeviceAddressHandle;

	/** Property handle to the CommunicationType property */
	TSharedPtr<IPropertyHandle> CommunicationTypeHandle;

	/** Property handle to the LocalUniverseStart property */
	TSharedPtr<IPropertyHandle> LocalUniverseStartHandle;

	/** Property handle to the ExternUniverseStart property */
	TSharedPtr<IPropertyHandle> ExternUniverseStartHandle;

	/** Property handle to the IsExternUniverseStartEditable property */
	TSharedPtr<IPropertyHandle> IsExternUniverseStartEditableHandle;

	/** Property handle to the PortGuid property */
	TSharedPtr<IPropertyHandle> PortGuidHandle;

	/** ComboBox that exposes a selection of communication types to the user */
	TSharedPtr<SDMXCommunicationTypeComboBox> CommunicationTypeComboBox;

	/** ComboBox to select a protocol name */
	TSharedPtr<SDMXProtocolNameComboBox> ProtocolNameComboBox;

	/** ComboBox that displays the device address when it is editable */
	TSharedPtr<SDMXIPAddressEditWidget> IPAddressEditWidget;

	/** Text Block that displays the device address when it is auto completed */
	TSharedPtr<STextBlock> AutoCompletedDeviceAddressTextBlock;

	/** Property utilities for this customization */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};

