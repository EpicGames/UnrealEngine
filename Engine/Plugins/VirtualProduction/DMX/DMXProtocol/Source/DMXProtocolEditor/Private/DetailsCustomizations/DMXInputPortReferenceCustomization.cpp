// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXInputPortReferenceCustomization.h"

#include "IO/DMXInputPortReference.h"

#include "PropertyHandle.h"


TSharedRef<IPropertyTypeCustomization> FDMXInputPortReferenceCustomization::MakeInstance()
{
	return MakeShared<FDMXInputPortReferenceCustomization>();
}

void FDMXInputPortReferenceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FDMXPortReferenceCustomizationBase::CustomizeHeader(StructPropertyHandle, HeaderRow, StructCustomizationUtils);}

void FDMXInputPortReferenceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PortGuidHandle = StructPropertyHandle->GetChildHandle(FDMXInputPortReference::GetPortGuidPropertyName());
	check(PortGuidHandle.IsValid());

	FDMXPortReferenceCustomizationBase::CustomizeChildren(StructPropertyHandle, ChildBuilder, StructCustomizationUtils);
}

const TSharedPtr<IPropertyHandle>& FDMXInputPortReferenceCustomization::GetPortGuidHandle() const
{
	return PortGuidHandle;
}

