// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPortReferenceCustomizationBase.h"

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FDMXPort;
class SDMXPortSelector;
class UDMXLibrary;

class SBorder;


/** Details customization for input and Input port references. */
class FDMXInputPortReferenceCustomization
	: public FDMXPortReferenceCustomizationBase
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:
	// ~Begin IPropertyTypecustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// ~End IPropertyTypecustomization Interface

	// ~Begin FDMXPortReferenceCustomizationBase Interface
	virtual bool IsInputPort() const override { return true; }
	virtual const TSharedPtr<IPropertyHandle>& GetPortGuidHandle() const override;
	// ~End DMXPortReferenceCustomizationBase Interface

	/** Property handle to the PortGuid property */
	TSharedPtr<IPropertyHandle> PortGuidHandle;
};
