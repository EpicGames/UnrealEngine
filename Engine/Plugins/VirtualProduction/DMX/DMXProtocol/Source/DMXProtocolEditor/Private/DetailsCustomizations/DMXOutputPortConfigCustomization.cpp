// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXOutputPortConfigCustomization.h"

#include "DetailWidgetRow.h"
#include "DMXProtocolModule.h"
#include "DMXProtocolSettings.h"
#include "DMXProtocolTypes.h"
#include "DMXProtocolUtils.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h" 
#include "IPropertyUtilities.h"
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXOutputPortConfig.h"
#include "IO/DMXPortManager.h"
#include "Misc/Guid.h" 
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SDMXCommunicationTypeComboBox.h"
#include "Widgets/SDMXDelayEditWidget.h"
#include "Widgets/SDMXIPAddressEditWidget.h"
#include "Widgets/SDMXProtocolNameComboBox.h"
#include "Widgets/Text/STextBlock.h" 


#define LOCTEXT_NAMESPACE "DMXOutputPortConfigCustomization"

TSharedRef<IPropertyTypeCustomization> FDMXOutputPortConfigCustomization::MakeInstance()
{
	return MakeShared<FDMXOutputPortConfigCustomization>();
}

void FDMXOutputPortConfigCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

void FDMXOutputPortConfigCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!StructPropertyHandle->IsExpanded())
	{	
		StructPropertyHandle->SetExpanded(true);
	}

	// Retrieve structure's child properties
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);
	TMap<FName, TSharedPtr< IPropertyHandle>> PropertyHandles;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();

		PropertyHandles.Add(PropertyName, ChildHandle);
	}

	// Cache customized properties
	ProtocolNameHandle = PropertyHandles.FindChecked(FDMXOutputPortConfig::GetProtocolNamePropertyNameChecked());
	CommunicationTypeHandle = PropertyHandles.FindChecked(FDMXOutputPortConfig::GetCommunicationTypePropertyNameChecked());
	AutoCompleteDeviceAddressEnabledHandle = PropertyHandles.FindChecked(FDMXOutputPortConfig::GetAutoCompleteDeviceAddressEnabledPropertyNameChecked());
	AutoCompleteDeviceAddressHandle = PropertyHandles.FindChecked(FDMXOutputPortConfig::GetAutoCompleteDeviceAddressPropertyNameChecked());
	DeviceAddressHandle = PropertyHandles.FindChecked(FDMXOutputPortConfig::GetDeviceAddressPropertyNameChecked());
	DelayHandle = PropertyHandles.FindChecked(FDMXOutputPortConfig::GetDelayPropertyNameChecked());
	DelayFrameRateHandle = PropertyHandles.FindChecked(FDMXOutputPortConfig::GetDelayFrameRatePropertyNameChecked());
	LocalUniverseStartHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetLocalUniverseStartPropertyNameChecked());
	ExternUniverseStartHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetExternUniverseStartPropertyNameChecked());
	IsExternUniverseStartEditableHandle = PropertyHandles.FindChecked(FDMXInputPortConfig::GetIsExternUnivereStartEditablePropertyNameChecked());
	PortGuidHandle = PropertyHandles.FindChecked(FDMXOutputPortConfig::GetPortGuidPropertyNameChecked());

	// Hande property changes
	AutoCompleteDeviceAddressHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXOutputPortConfigCustomization::UpdateAutoCompleteDeviceAddressTextBox));
	LocalUniverseStartHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXOutputPortConfigCustomization::OnLocalUniverseStartChanged));
	IsExternUniverseStartEditableHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXOutputPortConfigCustomization::OnIsExternUniverseStartEditableChanged));

	// Ports always need a valid Guid (cannot be blueprinted)
	if (!GetPortGuid().IsValid())
	{
		ChildBuilder.AddCustomRow(LOCTEXT("InvalidPortConfigSearchString", "Invalid"))
			.WholeRowContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("InvalidPortConfigText", "Invalid Port Guid. Cannot utilize this port."))
			];

		return;
	}

	// Add customized properties
	for (auto Iter(PropertyHandles.CreateConstIterator()); Iter; ++Iter)
	{
		// Don't add the PortGuid and the DelayFrameRate properties
		if (Iter.Value() == PortGuidHandle ||
			Iter.Value() == DelayFrameRateHandle)
		{
			continue;
		}

		// Don't add the bAutoCompeteDeviceAddressEnabled property
		// Don't add the bIsExternUniverseStartEditable property
		if (Iter.Value() == AutoCompleteDeviceAddressEnabledHandle ||
			Iter.Value() == IsExternUniverseStartEditableHandle)
		{
			continue;
		}

		// Add the property
		IDetailPropertyRow& PropertyRow = ChildBuilder.AddProperty(Iter.Value().ToSharedRef());

		if (Iter.Value() == ProtocolNameHandle)
		{
			GenerateProtocolNameRow(PropertyRow);
		}
		else if (Iter.Value() == CommunicationTypeHandle)
		{
			GenerateCommunicationTypeRow(PropertyRow);
		}
		else if (Iter.Value() == AutoCompleteDeviceAddressHandle)
		{
			GenerateAutoCompleteDeviceAddressRow(PropertyRow);
		}
		else if (Iter.Value() == DeviceAddressHandle)
		{
			GenerateDeviceAddressRow(PropertyRow);
		}
		else if (Iter.Value() == ExternUniverseStartHandle)
		{
			GenerateExternUniverseStartRow(PropertyRow);
		}
		else if (Iter.Key() == FDMXOutputPortConfig::GetDestinationAddressesPropertyNameChecked())
		{
			UpdateDestinationAddressesVisibility();

			TAttribute<EVisibility> DestinationAddressesVisibilityAttribute = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this]() 
				{
					return DestinationAddressesVisibility;
				}));

			PropertyRow.Visibility(DestinationAddressesVisibilityAttribute);
		}
		else if (Iter.Key() == FDMXOutputPortConfig::GetPriorityPropertyNameChecked())
		{
			TAttribute<EVisibility> PriorityVisibilityAttribute = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this]() 
				{
					const bool bPriorityIsVisible = GetProtocol().IsValid() && GetProtocol()->SupportsPrioritySettings();
					return bPriorityIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
				}));

			PropertyRow.Visibility(PriorityVisibilityAttribute);
		}
		else if (Iter.Key() == FDMXOutputPortConfig::GetDelayPropertyNameChecked())
		{
			GenerateDelayRow(PropertyRow);
		}
	}
}

void FDMXOutputPortConfigCustomization::GenerateProtocolNameRow(IDetailPropertyRow& PropertyRow)
{
	// Customizate the Protocol Name property to draw a combo box with all protocol names

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);
	
	const FName InitialSelection = [this]() -> FName
	{
		if (const IDMXProtocolPtr& Protocol = GetProtocol())
		{
			return Protocol->GetProtocolName();
		}

		return NAME_None;
	}();

	PropertyRow.CustomWidget()
		.NameContent()
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		[
			SAssignNew(ProtocolNameComboBox, SDMXProtocolNameComboBox)
			.InitiallySelectedProtocolName(InitialSelection)
			.OnProtocolNameSelected(this, &FDMXOutputPortConfigCustomization::OnProtocolNameSelected)
		];

}

void FDMXOutputPortConfigCustomization::GenerateCommunicationTypeRow(IDetailPropertyRow& PropertyRow)
{
	// Customizate the Communication Type property depending on the port's supported communication modes
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	PropertyRow.CustomWidget()
		.Visibility(TAttribute<EVisibility>::CreateLambda([this]()
			{
				if (CommunicationTypeComboBox.IsValid() && GetProtocol().IsValid())
				{
					return CommunicationTypeComboBox->GetVisibility();
				}
				return EVisibility::Collapsed;
			}))
		.NameContent()
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		[
			SAssignNew(CommunicationTypeComboBox, SDMXCommunicationTypeComboBox)
			.CommunicationTypes(GetSupportedCommunicationTypes())
			.InitialCommunicationType(GetCommunicationType())
			.OnCommunicationTypeSelected(this, &FDMXOutputPortConfigCustomization::OnCommunicationTypeSelected)
		];
}

void FDMXOutputPortConfigCustomization::GenerateDeviceAddressRow(IDetailPropertyRow& PropertyRow)
{
	// Customizate the IPAddress property to show a combo box with available IP addresses
	FString InitialValue = GetIPAddress();

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	// Override the value widget so it either shows 
	// - The IP Address Edit widget when auto complete is disabled
	// - The auto completed IP Address when auto complete is enabled
	ValueWidget =
		SNew(SHorizontalBox)
		
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(IPAddressEditWidget, SDMXIPAddressEditWidget)
			.Visibility_Lambda([this]()
				{
					return IsAutoCompleteDeviceAddressEnabled() ?
						EVisibility::Collapsed :
						EVisibility::Visible;
				})
			.InitialValue(InitialValue)
			.bShowLocalNICComboBox(true)
			.OnIPAddressSelected(this, &FDMXOutputPortConfigCustomization::OnDeviceAddressSelected)
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(AutoCompletedDeviceAddressTextBlock, STextBlock)
			.Visibility_Lambda([this]()
				{
					return IsAutoCompleteDeviceAddressEnabled() ?
						EVisibility::Visible :
						EVisibility::Collapsed;
				})
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		];
	UpdateAutoCompleteDeviceAddressTextBox();

	PropertyRow.IsEnabled(TAttribute<bool>::CreateLambda([this]
		{
			return !IsAutoCompleteDeviceAddressEnabled();
		}));

	PropertyRow.CustomWidget()
		.NameContent()
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		[
			ValueWidget.ToSharedRef()
		];
}	

void FDMXOutputPortConfigCustomization::GenerateAutoCompleteDeviceAddressRow(IDetailPropertyRow& PropertyRow)
{
	// Mimic a Edit Condition Toggle - Doing it directly via Meta Specifiers is not possible. -
	// The Edit Condition Toggle property wouldn't get properly serialized when being a Config property.
	const TSharedRef<SWidget> AutoCompleteDeviceAddressEnableldPropertyValueWidget = AutoCompleteDeviceAddressEnabledHandle->CreatePropertyValueWidget();

	const TSharedRef<SWidget> AutoCompleteDeviceAddressPropertyNameWidget = AutoCompleteDeviceAddressHandle->CreatePropertyNameWidget();
	const TSharedRef<SWidget> AutoCompleteDeviceAddressPropertyValueWidget = AutoCompleteDeviceAddressHandle->CreatePropertyValueWidget();

	AutoCompleteDeviceAddressPropertyNameWidget->SetEnabled(TAttribute<bool>::CreateLambda([this]()
		{
			return IsAutoCompleteDeviceAddressEnabled();
		}));
	AutoCompleteDeviceAddressPropertyValueWidget->SetEnabled(TAttribute<bool>::CreateLambda([this]()
		{
			return IsAutoCompleteDeviceAddressEnabled();
		}));

	PropertyRow.CustomWidget()
		.NameContent()
		[
			SNew(SHorizontalBox)

		// Edit Condition Toggle Checkbox
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Fill)
		.AutoWidth()
		[
			AutoCompleteDeviceAddressEnableldPropertyValueWidget
		]

	// Property Name
	+ SHorizontalBox::Slot()
		.Padding(8.f, 0.f, 0.f, 0.f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Fill)
		.AutoWidth()
		[
			AutoCompleteDeviceAddressPropertyNameWidget
		]
		]
	.ValueContent()
		[
			// Property Value
			AutoCompleteDeviceAddressPropertyValueWidget
		];
}

void FDMXOutputPortConfigCustomization::UpdateAutoCompleteDeviceAddressTextBox()
{
	// Set text on the AutoCompleteDeviceAddress Text Box depending on what the resulting IP is
	if (AutoCompletedDeviceAddressTextBlock.IsValid())
	{
		FString AutoCompleteDeviceAddress;
		if (AutoCompleteDeviceAddressHandle->GetValue(AutoCompleteDeviceAddress) == FPropertyAccess::Success)
		{
			FString NetworkInterfaceCardIPAddress;
			if (FDMXProtocolUtils::FindLocalNetworkInterfaceCardIPAddress(AutoCompleteDeviceAddress, NetworkInterfaceCardIPAddress))
			{
				const FText IPAddressText = FText::FromString(*NetworkInterfaceCardIPAddress);
				AutoCompletedDeviceAddressTextBlock->SetText(FText::Format(LOCTEXT("AutoCompletedIPAddressText", "{0} (auto-completed)"), IPAddressText));
			}
			else
			{
				// Fall back to the manually set one
				FString DeviceAddress;
				DeviceAddressHandle->GetValue(DeviceAddress);
				const FText NoMatchForIPAddressText = FText::Format(LOCTEXT("NoRegexMatchForDeviceAddress", "No match. Using '{0}'"), FText::FromString(DeviceAddress));
				AutoCompletedDeviceAddressTextBlock->SetText(NoMatchForIPAddressText);
			}
		}
	}
}

void FDMXOutputPortConfigCustomization::GenerateExternUniverseStartRow(IDetailPropertyRow& PropertyRow)
{
	const TAttribute<bool> CanEditExternUniverseAttribute = TAttribute<bool>::CreateLambda([this]()
		{
			bool bIsExternUniverseStartEditable;
			if (IsExternUniverseStartEditableHandle->GetValue(bIsExternUniverseStartEditable) == FPropertyAccess::Success)
			{
				return bIsExternUniverseStartEditable ? true : false;
			}
			return false;
		});

	TSharedPtr<SWidget> ExternUniverseStartNameWidget;
	TSharedPtr<SWidget> ExternUniverseStartValueWidget;
	FDetailWidgetRow ExternUniverseStartRow;
	PropertyRow.GetDefaultWidgets(ExternUniverseStartNameWidget, ExternUniverseStartValueWidget, ExternUniverseStartRow);

	const FText IsExternUniverseEnabledTooltip = LOCTEXT("IsExternUniverseEnabledWarningTooltip", "Remaps the range of local Universes (Num Universes from Local Universe Start) to a different set of Universes at the Protocol level.\nFor example, for Art-Net this allows to map local Universe 1 to extern Universe 0.");

	PropertyRow.CustomWidget()
		.NameContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 4.f, 0.f)
			[
				SNew(SCheckBox)
				.ToolTipText(IsExternUniverseEnabledTooltip)
				.IsChecked_Lambda([CanEditExternUniverseAttribute]()
					{
						return CanEditExternUniverseAttribute.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
					{
						const bool bIsExternUniverseStartEditable = NewState == ECheckBoxState::Checked;
						IsExternUniverseStartEditableHandle->SetValue(bIsExternUniverseStartEditable);
					})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.IsEnabled(CanEditExternUniverseAttribute)
				[
					ExternUniverseStartNameWidget.ToSharedRef()
				]
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			.IsEnabled(CanEditExternUniverseAttribute)

			+ SHorizontalBox::Slot()
			.Padding(0.f, 0.f, 4.f, 0.f)
			[
				ExternUniverseStartValueWidget.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.f)
			[
				SNew(SImage)
				.Visibility_Lambda([CanEditExternUniverseAttribute]()
					{
						return CanEditExternUniverseAttribute.Get() ? EVisibility::Visible : EVisibility::Collapsed;
					})
				.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
				.ToolTipText(IsExternUniverseEnabledTooltip)
			]
		];
}

void FDMXOutputPortConfigCustomization::OnLocalUniverseStartChanged()
{
	bool bCanEditExternUniverse;
	int32 LocalUniverse;
	if (IsExternUniverseStartEditableHandle->GetValue(bCanEditExternUniverse) == FPropertyAccess::Success &&
		!bCanEditExternUniverse &&
		LocalUniverseStartHandle->GetValue(LocalUniverse) == FPropertyAccess::Success)
	{
		ExternUniverseStartHandle->NotifyPreChange();
		// Change the extern universe interactive only. Rely on the local universe start change completing the transaction.
		ExternUniverseStartHandle->SetValue(LocalUniverse, EPropertyValueSetFlags::InteractiveChange);
		ExternUniverseStartHandle->NotifyPostChange(EPropertyChangeType::Interactive);
	}
}

void FDMXOutputPortConfigCustomization::OnIsExternUniverseStartEditableChanged()
{
	bool bCanEditExternUniverse;
	int32 LocalUniverse;
	if (IsExternUniverseStartEditableHandle->GetValue(bCanEditExternUniverse) == FPropertyAccess::Success &&
		!bCanEditExternUniverse &&
		LocalUniverseStartHandle->GetValue(LocalUniverse) == FPropertyAccess::Success)
	{
		ExternUniverseStartHandle->NotifyPreChange();
		ExternUniverseStartHandle->SetValue(LocalUniverse);
		ExternUniverseStartHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

void FDMXOutputPortConfigCustomization::GenerateDelayRow(IDetailPropertyRow& PropertyRow)
{
	// Customizate the IPAddress property to show a combo box with available IP addresses
	const double InitialDelay = GetDelay();
	const TArray<FFrameRate> InitialDelayFrameRates = GetDelayFrameRates();
	
	FFrameRate InitialDelayFrameRate;
	if (ensureMsgf(InitialDelayFrameRates.Num() == 1, TEXT("Multi editing Output Port Configs is not supported.")))
	{
		InitialDelayFrameRate = InitialDelayFrameRates[0];
	}

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	PropertyRow.CustomWidget()
		.NameContent()
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		[
			SAssignNew(DelayEditWidget, SDMXDelayEditWidget)
			.InitialDelay(InitialDelay)
			.InitialDelayFrameRate(InitialDelayFrameRate)
			.OnDelayChanged(this, &FDMXOutputPortConfigCustomization::OnDelayChanged)
			.OnDelayFrameRateChanged(this, &FDMXOutputPortConfigCustomization::OnDelayFrameRateChanged)
		];
}

void FDMXOutputPortConfigCustomization::OnProtocolNameSelected()
{
	FName ProtocolName = ProtocolNameComboBox->GetSelectedProtocolName();

	const FScopedTransaction Transaction(LOCTEXT("ProtocolSelected", "Select DMX Output Port Protocol"));

	ProtocolNameHandle->NotifyPreChange();
	ProtocolNameHandle->SetValue(ProtocolName);
	ProtocolNameHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	PropertyUtilities->ForceRefresh();
}

void FDMXOutputPortConfigCustomization::OnCommunicationTypeSelected()
{
	EDMXCommunicationType SelectedCommunicationType = CommunicationTypeComboBox->GetSelectedCommunicationType();

	const FScopedTransaction Transaction(LOCTEXT("CommunicationTypeSelected", "Select DMX Output Port Communication Type"));

	CommunicationTypeHandle->NotifyPreChange();
	CommunicationTypeHandle->SetValue(static_cast<uint8>(SelectedCommunicationType));
	CommunicationTypeHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	PropertyUtilities->ForceRefresh();
}

void FDMXOutputPortConfigCustomization::OnDeviceAddressSelected()
{
	FString SelectedIP = IPAddressEditWidget->GetSelectedIPAddress();

	const FScopedTransaction Transaction(LOCTEXT("IPAddressSelected", "Selected DMX Output Port IP Address"));

	DeviceAddressHandle->NotifyPreChange();
	ensure(DeviceAddressHandle->SetValue(SelectedIP) == FPropertyAccess::Success);
	DeviceAddressHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

void FDMXOutputPortConfigCustomization::UpdateDestinationAddressesVisibility()
{
	EDMXCommunicationType CommunicationType = GetCommunicationType();
	if (CommunicationType == EDMXCommunicationType::Unicast)
	{
		DestinationAddressesVisibility = EVisibility::Visible;
	}
	else
	{
		DestinationAddressesVisibility = EVisibility::Collapsed;
	}
}

void FDMXOutputPortConfigCustomization::OnDelayChanged()
{
	// Clamp the delay value to a reasonable range
	static const double MaxDelaySeconds = 60.f;

	const double DesiredDelaySeconds = DelayEditWidget->GetDelay() / DelayEditWidget->GetDelayFrameRate().AsDecimal();
	const double NewDelay = DesiredDelaySeconds > MaxDelaySeconds ? MaxDelaySeconds * DelayEditWidget->GetDelayFrameRate().AsDecimal() : DelayEditWidget->GetDelay();

	const FScopedTransaction Transaction(LOCTEXT("DelalySecondsChange", "Set DMX Output Port Delay"));

	DelayHandle->NotifyPreChange();
	if (DelayHandle->SetValue(NewDelay) == FPropertyAccess::Success)
	{
		DelayHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyUtilities->ForceRefresh();
	}
}

void FDMXOutputPortConfigCustomization::OnDelayFrameRateChanged()
{
	const FFrameRate DelayFrameRate = DelayEditWidget->GetDelayFrameRate();

	const FScopedTransaction Transaction(LOCTEXT("DelayFrameRateChanged", "Set DMX Output Port Delay Type"));

	DeviceAddressHandle->NotifyPreChange();

	TArray<void*> RawDatas;
	DelayFrameRateHandle->AccessRawData(RawDatas);

	for (void* RawData : RawDatas)
	{
		FFrameRate* FrameRatePtr = (FFrameRate*)RawData;
		*FrameRatePtr = DelayFrameRate;
	}
	DeviceAddressHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	// Clamp the delay value to a reasonable range
	static const double MaxDelaySeconds = 60.f;

	const double DesiredDelaySeconds = DelayEditWidget->GetDelay() / DelayFrameRate.AsDecimal();
	const double NewDelay = DesiredDelaySeconds > MaxDelaySeconds ? MaxDelaySeconds * DelayFrameRate.AsDecimal() : DelayEditWidget->GetDelay();

	DelayHandle->NotifyPreChange();
	if (DelayHandle->SetValue(NewDelay) == FPropertyAccess::Success)
	{
		DelayHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyUtilities->ForceRefresh();
	}
}

double FDMXOutputPortConfigCustomization::GetDelay() const
{
	double Delay;
	if (DelayHandle->GetValue(Delay) == FPropertyAccess::Success)
	{
		return Delay;
	}

	return 0.0;
}

TArray<FFrameRate> FDMXOutputPortConfigCustomization::GetDelayFrameRates() const
{
	TArray<FFrameRate> FrameRates;

	TArray<void*> RawDatas;
	DelayFrameRateHandle->AccessRawData(RawDatas);

	for (void* RawData : RawDatas)
	{
		const FFrameRate* FrameRatePtr = (FFrameRate*)RawData;
		if (FrameRatePtr)
		{
			FrameRates.Add(*FrameRatePtr);
		}
	}

	return FrameRates;
}

IDMXProtocolPtr FDMXOutputPortConfigCustomization::GetProtocol() const
{
	FName ProtocolName;
	ProtocolNameHandle->GetValue(ProtocolName);

	IDMXProtocolPtr Protocol = IDMXProtocol::Get(ProtocolName);
	
	return Protocol;
}

FGuid FDMXOutputPortConfigCustomization::GetPortGuid() const
{
	TArray<void*> RawData;
	PortGuidHandle->AccessRawData(RawData);

	// Multiediting is not supported, may fire if this is used in a blueprint way that would support it
	if (ensureMsgf(RawData.Num() == 1, TEXT("Using port config in ways that would enable multiediting is not supported.")))
	{
		const FGuid* PortGuidPtr = reinterpret_cast<FGuid*>(RawData[0]);
		if (PortGuidPtr && PortGuidPtr->IsValid())
		{
			return *PortGuidPtr;
		}
	}

	return FGuid();
}

const TArray<EDMXCommunicationType> FDMXOutputPortConfigCustomization::GetSupportedCommunicationTypes() const
{
	if (IDMXProtocolPtr Protocol = GetProtocol())
	{
		return Protocol->GetOutputPortCommunicationTypes();
	}

	return TArray<EDMXCommunicationType>();
}

EDMXCommunicationType FDMXOutputPortConfigCustomization::GetCommunicationType() const
{
	uint8 CommunicationType;
	ensure(CommunicationTypeHandle->GetValue(CommunicationType) == FPropertyAccess::Success);

	return static_cast<EDMXCommunicationType>(CommunicationType);
}

bool FDMXOutputPortConfigCustomization::IsAutoCompleteDeviceAddressEnabled() const
{
	bool bAutoCompleteDeviceAddressEnabled;
	AutoCompleteDeviceAddressEnabledHandle->GetValue(bAutoCompleteDeviceAddressEnabled);

	return bAutoCompleteDeviceAddressEnabled;
}

FString FDMXOutputPortConfigCustomization::GetIPAddress() const
{
	FGuid PortGuid = GetPortGuid();
	if (PortGuid.IsValid())
	{
		const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
		checkf(ProtocolSettings, TEXT("Unexpected protocol settings not available when its details are customized"));

		const FDMXInputPortConfig* InputPortConfigPtr = ProtocolSettings->InputPortConfigs.FindByPredicate([&PortGuid](const FDMXInputPortConfig& InputPortConfig)
			{
				return InputPortConfig.GetPortGuid() == PortGuid;
			});
		if (InputPortConfigPtr)
		{
			return InputPortConfigPtr->GetDeviceAddress();
		}

		const FDMXOutputPortConfig* OutputPortConfigPtr = ProtocolSettings->OutputPortConfigs.FindByPredicate([&PortGuid](const FDMXOutputPortConfig& OutputPortConfig)
			{
				return OutputPortConfig.GetPortGuid() == PortGuid;
			});
		if (OutputPortConfigPtr)
		{
			return OutputPortConfigPtr->GetDeviceAddress();
		}
	}

	FString IPAddress;
	ensure(DeviceAddressHandle->GetValue(IPAddress) == FPropertyAccess::Success);

	return IPAddress;
}

#undef LOCTEXT_NAMESPACE
