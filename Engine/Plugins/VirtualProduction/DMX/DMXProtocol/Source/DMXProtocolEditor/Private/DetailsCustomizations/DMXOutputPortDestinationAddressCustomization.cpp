// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXOutputPortDestinationAddressCustomization.h"

#include "IO/DMXOutputPortConfig.h"
#include "Widgets/SDMXIPAddressEditWidget.h"

#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h" 
#include "DetailWidgetRow.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"


#define LOCTEXT_NAMESPACE "DMXOutputPortDestinationAddressCustomization"

TSharedRef<IPropertyTypeCustomization> FDMXOutputPortDestinationAddressCustomization::MakeInstance()
{
	return MakeShared<FDMXOutputPortDestinationAddressCustomization>();
}

void FDMXOutputPortDestinationAddressCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Nothing in the header
}

void FDMXOutputPortDestinationAddressCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	DestinationAddressStringHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FDMXOutputPortDestinationAddress, DestinationAddressString));

	// Create the content depending on the property being an array member 
	TSharedRef<SHorizontalBox> ContentBox = SNew(SHorizontalBox);

	ContentBox->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.MinDesiredWidth(124.f)
			[
				SAssignNew(IPAddressEditWidget, SDMXIPAddressEditWidget)
				.InitialValue(GetIPAddress())
				.bShowLocalNICComboBox(false)
				.OnIPAddressSelected(this, &FDMXOutputPortDestinationAddressCustomization::OnDestinationAddressSelected)
			]
		];

	const bool bIsArrayMember = [this]()
	{
		if (const TSharedPtr<IPropertyHandle> DestinationAddressStructHandle = DestinationAddressStringHandle->GetParentHandle())
		{
			if (TSharedPtr<IPropertyHandle> DestinationAddressesHandle = DestinationAddressStructHandle->GetParentHandle())
			{
				if (TSharedPtr<IPropertyHandleArray> DestinationAddressesHandleArray = DestinationAddressesHandle->AsArray())
				{
					return true;
				}
			}
		}

		return false;
	}();

	if (bIsArrayMember)
	{
		ContentBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.ContentPadding(2.0f)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &FDMXOutputPortDestinationAddressCustomization::OnDeleteDestinationAddressClicked)
				.ToolTipText(LOCTEXT("RemoveCellAttributeTooltip", "Removes the Destination Address"))
				.ForegroundColor(FSlateColor::UseForeground())
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
					.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
				]
			];
	}

	ChildBuilder.AddCustomRow(LOCTEXT("InvalidPortConfigSearchString", "Invalid"))
		.ValueContent()
		[
			ContentBox
		];
}

FString FDMXOutputPortDestinationAddressCustomization::GetIPAddress() const
{
	FString IPAddress;
	if (DestinationAddressStringHandle->GetValue(IPAddress) == FPropertyAccess::Success)
	{
		return IPAddress;
	}

	return TEXT("");
}

void FDMXOutputPortDestinationAddressCustomization::SetIPAddress(const FString& NewIPAddress)
{
	DestinationAddressStringHandle->NotifyPreChange();
	DestinationAddressStringHandle->SetValue(NewIPAddress);
	DestinationAddressStringHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

void FDMXOutputPortDestinationAddressCustomization::OnDestinationAddressSelected()
{
	TArray<UObject*> OuterObjects;
	DestinationAddressStringHandle->GetOuterObjects(OuterObjects);
	
	// Forward the change to the outer object so it gets the property change. Required to get the change from the nested property all the way to the property owner (e.g. UDMXProtocolSettings)
	for (UObject* Object : OuterObjects)
	{
		Object->PreEditChange(FDMXOutputPortDestinationAddress::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXOutputPortDestinationAddress, DestinationAddressString)));
	}

	const FString IPAddress = IPAddressEditWidget->GetSelectedIPAddress();
	SetIPAddress(IPAddress);

	for (UObject* Object : OuterObjects)
	{
		Object->PostEditChange();
	}
}

FReply FDMXOutputPortDestinationAddressCustomization::OnDeleteDestinationAddressClicked()
{
	if (const TSharedPtr<IPropertyHandle> DestinationAddressStructHandle = DestinationAddressStringHandle->GetParentHandle())
	{
		if (TSharedPtr<IPropertyHandle> DestinationAddressesHandle = DestinationAddressStructHandle->GetParentHandle())
		{
			if (TSharedPtr<IPropertyHandleArray> DestinationAddressesHandleArray = DestinationAddressesHandle->AsArray())
			{
				// Find the index of this property
				const uint32 MyIndexInArray = [&DestinationAddressStructHandle, &DestinationAddressesHandleArray, this]() -> uint32
				{
					uint32 NumElements;
					DestinationAddressesHandleArray->GetNumElements(NumElements);
					for (uint32 Index = 0; Index < NumElements; Index++)
					{
						if (DestinationAddressStructHandle->GetIndexInArray() == Index)
						{
							return Index;
						}
					}

					return INDEX_NONE;
				}();

				if (MyIndexInArray != INDEX_NONE)
				{
					TArray<UObject*> OuterObjects;
					DestinationAddressStringHandle->GetOuterObjects(OuterObjects);

					// Forward the change to the outer object so it gets the property change. Required to get the change from the nested property all the way to the property owner (e.g. UDMXProtocolSettings)
					for (UObject* Object : OuterObjects)
					{
						Object->PreEditChange(FDMXOutputPortDestinationAddress::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(FDMXOutputPortDestinationAddress, DestinationAddressString)));
					}

					DestinationAddressesHandleArray->DeleteItem(MyIndexInArray);

					for (UObject* Object : OuterObjects)
					{
						Object->PostEditChange();
					}
				}
			}
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
