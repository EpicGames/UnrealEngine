// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"

class FReply;
class IPropertyHandle;
class SDMXIPAddressEditWidget;


/** Customization for the DMXOutputPortConfigCustomization struct */
class FDMXOutputPortDestinationAddressCustomization
	: public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:
	// ~Begin IPropertyTypecustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// ~End IPropertyTypecustomization Interface

private:
	/** Gets the IP Address currently set */
	FString GetIPAddress() const;

	/** Sets the IP Adderss */
	void SetIPAddress(const FString& NewIPAddress);

	/** Called when a destination Address was selected */
	void OnDestinationAddressSelected();

	/** Called when a destination address wants to be deleted */
	FReply OnDeleteDestinationAddressClicked();

	/** Widget to enter an IP address */
	TSharedPtr<SDMXIPAddressEditWidget> IPAddressEditWidget;

	/** Property handle for the DestinationAddressString property */
	TSharedPtr<IPropertyHandle> DestinationAddressStringHandle;
};
