// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXOutputPortReferenceCustomization.h"

#include "IO/DMXOutputPortReference.h"

#include "PropertyHandle.h"


TSharedRef<IPropertyTypeCustomization> FDMXOutputPortReferenceCustomization::MakeInstance()
{
	return MakeShared<FDMXOutputPortReferenceCustomization>();
}

void FDMXOutputPortReferenceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FDMXPortReferenceCustomizationBase::CustomizeHeader(StructPropertyHandle, HeaderRow, StructCustomizationUtils);
}

void FDMXOutputPortReferenceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PortGuidHandle = StructPropertyHandle->GetChildHandle(FDMXOutputPortReference::GetPortGuidPropertyName());
	check(PortGuidHandle.IsValid());

	FDMXPortReferenceCustomizationBase::CustomizeChildren(StructPropertyHandle, ChildBuilder, StructCustomizationUtils);
}

const TSharedPtr<IPropertyHandle>& FDMXOutputPortReferenceCustomization::GetPortGuidHandle() const
{
	return PortGuidHandle;
}
