// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPortReferenceCustomizationBase.h"

#include "DMXProtocolLog.h"
#include "DMXProtocolCommon.h"
#include "ScopedTransaction.h" 
#include "Interfaces/IDMXProtocol.h"
#include "IO/DMXPortManager.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "Widgets/SDMXPortSelector.h"

#include "DetailWidgetRow.h"
#include "Styling/AppStyle.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h" 
#include "IPropertyUtilities.h"
#include "Misc/Guid.h" 
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h" 

#define LOCTEXT_NAMESPACE "DMXPortConfigCustomizationBase"

FDMXPortReferenceCustomizationBase::FDMXPortReferenceCustomizationBase()
{}

void FDMXPortReferenceCustomizationBase::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	constexpr TCHAR ShowOnlyInnerPropertiesMetaDataSpecifier[] = TEXT("ShowOnlyInnerProperties");
	bool bShowHeader = !StructPropertyHandle->HasMetaData(ShowOnlyInnerPropertiesMetaDataSpecifier);
	if (bShowHeader)
	{
		HeaderRow
			.NameContent()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				StructPropertyHandle->CreatePropertyValueWidget()
			];
	}
}

void FDMXPortReferenceCustomizationBase::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

	// Hide the 'reset to default' option
	StructPropertyHandle->MarkResetToDefaultCustomized();

	// Add the port selector row
	EDMXPortSelectorMode PortSelectorMode = IsInputPort() ? EDMXPortSelectorMode::SelectFromAvailableInputs : EDMXPortSelectorMode::SelectFromAvailableOutputs;

	ChildBuilder
		.AddCustomRow(LOCTEXT("PortSelectorSearchString", "Port"))
		.NameContent()
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.Text(LOCTEXT("PortLabel", "Port"))
		]
		.ValueContent()
		[
			SAssignNew(PortSelector, SDMXPortSelector)
			.Mode(PortSelectorMode)
			.OnPortSelected(this, &FDMXPortReferenceCustomizationBase::OnPortSelected)
		];

	// Four possible states here that all need be handled:
	// a) This is an existing port reference, with a corresponding port config
	// b) This is an existing port reference, but the corresponding port config got deleted in settings
	// c) This is a new port reference and doesn't point to any port yet
	// d) The port references are multiselected

	const TArray<FDMXPortSharedPtr> Ports = FindPortItems();
	if(Ports.Num() == 1 && Ports[0].IsValid())
	{
		// a) This is a single existing port reference, with a corresponding port config
		const FGuid PortGuid = Ports[0]->GetPortGuid();
		PortSelector->SelectPort(PortGuid);
	}
	else
	{
		const TArray<FGuid> PortGuids = GetPortGuids();
		if (PortGuids.Num() == 1 && PortGuids[0].IsValid())
		{
			// b) This is an existing port reference, but the corresponding port config got deleted in settings

			// Let the port remain invalid, but let users know
			UE_LOGF(LogDMXProtocol, Error, "The referenced Port was deleted from project settings.");
		}
		else if(PortGuids.Num() == 0 && PortSelector->HasSelection())
		{
			// c) This is a new port reference and doesn't point to any port yet
			//	  The port selector makes an initial selection already, so if it's valid there are ports
			//    It only needs be applied on the new reference.
			ApplySelectedPortGuid();
		}
		else
		{
			// d) This is multi selected in some way
			PortSelector->SetHasMultipleValues();
		}
	}

	FDMXPortManager::Get().OnPortsChanged.AddSP(this, &FDMXPortReferenceCustomizationBase::OnPortsChanged);
}

void FDMXPortReferenceCustomizationBase::OnPortSelected()
{
	ApplySelectedPortGuid();
}

void FDMXPortReferenceCustomizationBase::OnPortsChanged()
{
	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities->ForceRefresh();
	}
}

TArray<FDMXPortSharedPtr> FDMXPortReferenceCustomizationBase::FindPortItems() const
{
	const TArray<FGuid> PortGuids = GetPortGuids();

	TArray<FDMXPortSharedPtr> Result;
	for (const FGuid& PortGuid : PortGuids)
	{
		if (PortGuid.IsValid())
		{
			const FDMXInputPortSharedRef* InputPortPtr = FDMXPortManager::Get().GetInputPorts().FindByPredicate([&PortGuid](const FDMXInputPortSharedRef& InputPort) {
				return InputPort->GetPortGuid() == PortGuid;
				});

			if (InputPortPtr)
			{
				Result.Add(*InputPortPtr);
			}

			const FDMXOutputPortSharedRef* OutputPortPtr = FDMXPortManager::Get().GetOutputPorts().FindByPredicate([&PortGuid](const FDMXOutputPortSharedRef& OutputPort) {
				return OutputPort->GetPortGuid() == PortGuid;
				});

			if (OutputPortPtr)
			{
				Result.Add(*OutputPortPtr);
			}
		}
	}

	return Result;
}

void FDMXPortReferenceCustomizationBase::ApplySelectedPortGuid()
{
	const FGuid SelectedGuid = [this]()
	{
		if (IsInputPort())
		{
			if (const FDMXInputPortSharedPtr& InputPort = PortSelector->GetSelectedInputPort())
			{
				return InputPort->GetPortGuid();
			}
		}

		if (const FDMXOutputPortSharedPtr& OutputPort = PortSelector->GetSelectedOutputPort())
		{
			return OutputPort->GetPortGuid();
		}

		return FGuid();
	}();

	if (!GetPortGuids().Contains(SelectedGuid))
	{
		const TSharedPtr<IPropertyHandle>& PortGuidHandle = GetPortGuidHandle();
		if (!PortGuidHandle.IsValid() || !PortGuidHandle->IsValidHandle())
		{
			// May be invalid while port arrays are changing
			return;
		}

		PortGuidHandle->NotifyPreChange();

		TArray<void*> RawDataArray;
		PortGuidHandle->AccessRawData(RawDataArray);

		for (void* RawData : RawDataArray)
		{
			FMemory::Memcpy(RawData, &SelectedGuid, sizeof(FGuid));
		}

		PortGuidHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	}
}

TArray<FGuid> FDMXPortReferenceCustomizationBase::GetPortGuids() const
{
	const TSharedPtr<IPropertyHandle>& PortGuidHandle = GetPortGuidHandle();
	if (!PortGuidHandle.IsValid() || !PortGuidHandle->IsValidHandle())
	{			
		// May be invalid while port arrays are changing
		return TArray<FGuid>();
	}

	TArray<void*> RawDataArray;
	PortGuidHandle->AccessRawData(RawDataArray);

	TArray<FGuid> Result;
	for (const void* RawData : RawDataArray)
	{
		const FGuid* PortGuidPtr = reinterpret_cast<const FGuid*>(RawData);
		if (PortGuidPtr && PortGuidPtr->IsValid())
		{
			Result.Add(*PortGuidPtr);
		}
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE
