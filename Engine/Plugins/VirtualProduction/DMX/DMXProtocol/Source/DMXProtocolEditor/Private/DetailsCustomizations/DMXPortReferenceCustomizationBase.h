// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class FDMXPort;
class SDMXPortSelector;

class SBorder;
class IPropertyHandle;
class IPropertyUtilities;


/** Base class for input and output port reference details customizations. */
class FDMXPortReferenceCustomizationBase
	: public IPropertyTypeCustomization
{
protected:
	FDMXPortReferenceCustomizationBase();
	
	virtual ~FDMXPortReferenceCustomizationBase()
	{}

	// ~Begin IPropertyTypecustomization Interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// ~End IPropertyTypecustomization Interface

	/** Called to deduce if the child is an input (or an output) */
	virtual bool IsInputPort() const = 0;

	/** Called to get the PortGuid property handle */
	virtual const TSharedPtr<IPropertyHandle>& GetPortGuidHandle() const = 0;

private:
	/** Called when a port was selected in the port selector */
	void OnPortSelected();

	/** Called when ports changed */
	void OnPortsChanged();

	/** Helper function that finds the corresponding input port, or nullptr if no corresponding port */
	TArray<FDMXPortSharedPtr> FindPortItems() const;

	/** Applies the selected port Guid to the customized struct */
	void ApplySelectedPortGuid();

	/** Helper function that gets the Guid of the edited port(s) */
	TArray<FGuid> GetPortGuids() const;

	/** Port selector widget */
	TSharedPtr<SDMXPortSelector> PortSelector;

	/** Error text shown when the port couldn't be found, may contain the last error if there is no error - I11t is not a state. */
	FText ErrorText;

	/** Detail customization utils */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
