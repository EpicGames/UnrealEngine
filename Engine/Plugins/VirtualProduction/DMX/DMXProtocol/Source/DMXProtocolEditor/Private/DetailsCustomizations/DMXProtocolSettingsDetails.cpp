// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolSettingsDetails.h"

#include "DMXProtocolSettings.h"

#include "DetailLayoutBuilder.h"
#include "Interfaces/IPluginManager.h"


namespace UE::DMXProtocolEditor::Private::DMXProtocolSettingsDetails
{
	bool IsDMXEnginePluginEnabled()
	{
		static const FString DMXEnginePluginName = TEXT("DMXEngine");

		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(DMXEnginePluginName);
		return Plugin.IsValid() && Plugin->IsEnabled();
	}
}

FDMXProtocolSettingsDetails::FDMXProtocolSettingsDetails()
	: bIsDMXEnginePluginEnabled(UE::DMXProtocolEditor::Private::DMXProtocolSettingsDetails::IsDMXEnginePluginEnabled())
{}

TSharedRef<IDetailCustomization> FDMXProtocolSettingsDetails::MakeInstance()
{
	const TSharedRef<FDMXProtocolSettingsDetails> ProtocolSettingsDetails = MakeShared<FDMXProtocolSettingsDetails>();
	return ProtocolSettingsDetails;
}

void FDMXProtocolSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	if (!bIsDMXEnginePluginEnabled)
	{
		DetailBuilder.HideProperty(UDMXProtocolSettings::GetAllFixturePatchesReceiveDMXInEditorPropertyName());
	}
}
