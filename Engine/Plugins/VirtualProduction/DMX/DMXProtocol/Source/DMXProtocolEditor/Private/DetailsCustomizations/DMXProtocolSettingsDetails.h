// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"


/** Details customization for the DMMXFixturePatch class */
class FDMXProtocolSettingsDetails
	: public IDetailCustomization
{
public:
	/** Constructor */
	FDMXProtocolSettingsDetails();

	/** Creates a detail customization instance */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface

private:
	/** True if the DMX Engine Plugin is enabled */
	const bool bIsDMXEnginePluginEnabled;
};
