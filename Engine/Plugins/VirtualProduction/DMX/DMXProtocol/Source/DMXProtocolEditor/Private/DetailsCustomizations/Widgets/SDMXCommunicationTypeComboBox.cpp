// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXCommunicationTypeComboBox.h"

#include "DMXProtocolModule.h"

#include "Styling/AppStyle.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h" 


const TMap<EDMXCommunicationType, TSharedPtr<FString>> SDMXCommunicationTypeComboBox::CommunicationTypeToStringMap =
{
	TTuple<EDMXCommunicationType, TSharedPtr<FString>>(EDMXCommunicationType::Broadcast, MakeShared<FString>(TEXT("Broadcast"))),
	TTuple<EDMXCommunicationType, TSharedPtr<FString>>(EDMXCommunicationType::Multicast, MakeShared<FString>(TEXT("Multicast"))),
	TTuple<EDMXCommunicationType, TSharedPtr<FString>>(EDMXCommunicationType::Unicast,   MakeShared<FString>(TEXT("Unicast")))
	// InternalOnly missing on purpose, it is not an option exposed ot the user and should never be drawn
};

void SDMXCommunicationTypeComboBox::Construct(const FArguments& InArgs)
{
	OnCommunicationTypeSelected = InArgs._OnCommunicationTypeSelected;

	ChildSlot
		[
			SAssignNew(CommunicationTypeComboBox, SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&CommunicationTypesSource)
			.OnGenerateWidget(this, &SDMXCommunicationTypeComboBox::GenerateCommunicationTypeComboBoxEntry)
			.OnSelectionChanged(this, &SDMXCommunicationTypeComboBox::HandleCommunicationTypeSelectionChanged)
			.Content()
			[
				SAssignNew(CommunicationTypeTextBlock, STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
		];

	SetCommunicationTypesInternal(InArgs._CommunicationTypes);

	// Make an initial selection if the Communication Types contains a selection
	if (InArgs._InitialCommunicationType != EDMXCommunicationType::InternalOnly)
	{
		TSharedPtr<FString> InitialSelection = CommunicationTypeToStringMap.FindChecked(InArgs._InitialCommunicationType);
		
		if (CommunicationTypesSource.Contains(InitialSelection))
		{
			CommunicationTypeComboBox->SetSelectedItem(InitialSelection);
		}
		else if(CommunicationTypesSource.Num() > 0)
		{
			// Recover from a bad initial selection
			CommunicationTypeComboBox->SetSelectedItem(CommunicationTypesSource[0]);
		}
	}
}

void SDMXCommunicationTypeComboBox::SetCommunicationTypesInternal(const TArray<EDMXCommunicationType>& NewCommunicationTypes)
{
	check(CommunicationTypeComboBox.IsValid());
	check(CommunicationTypeTextBlock.IsValid());

	if (NewCommunicationTypes.Num() == 0 ||
		NewCommunicationTypes[0] == EDMXCommunicationType::InternalOnly)
	{
		// Hide the box when the selection shouldn't be exposed to the user
		CommunicationTypeComboBox->ClearSelection();
		
		SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		// We expect either a selection of valid Communication Types or InternalOnly, but never both
		check(!NewCommunicationTypes.Contains(EDMXCommunicationType::InternalOnly));

		GenerateOptionsSource(NewCommunicationTypes);

		TSharedPtr<FString> PreviousSelection = CommunicationTypeComboBox->GetSelectedItem();

		CommunicationTypeComboBox->RefreshOptions();

		// Try to restore the previous selection
		if (PreviousSelection.IsValid())
		{
			const TSharedPtr<FString>* ReselectableItemPtr = CommunicationTypesSource.FindByPredicate([&PreviousSelection](const TSharedPtr<FString>& SourceEntry) {
				return *SourceEntry == *PreviousSelection;
			});

			if (ReselectableItemPtr)
			{
				CommunicationTypeComboBox->SetSelectedItem(*ReselectableItemPtr);
			}
		}
		else
		{
			CommunicationTypeComboBox->SetSelectedItem(CommunicationTypesSource[0]);
		}

		TSharedPtr<FString> NewSelection = CommunicationTypeComboBox->GetSelectedItem();
		check(NewSelection.IsValid());

		CommunicationTypeTextBlock->SetText(FText::FromString(*NewSelection));
		SetVisibility(EVisibility::Visible);
	}
}

EDMXCommunicationType SDMXCommunicationTypeComboBox::GetSelectedCommunicationType() const
{
	check(CommunicationTypeComboBox.IsValid());

	TSharedPtr<FString> Selection = CommunicationTypeComboBox->GetSelectedItem();

	if (Selection.IsValid())
	{
		for (const TTuple<EDMXCommunicationType, TSharedPtr<FString>>& CommunicationTypeStringKvp : CommunicationTypeToStringMap)
		{
			if (CommunicationTypeStringKvp.Value == Selection)
			{
				return CommunicationTypeStringKvp.Key;
			}
		}
	}

	return EDMXCommunicationType::InternalOnly;
}

void SDMXCommunicationTypeComboBox::GenerateOptionsSource(const TArray<EDMXCommunicationType>& CommunicationTypes)
{
	CommunicationTypesSource.Reset();
	for (EDMXCommunicationType CommunicationType : CommunicationTypes)
	{
		if (CommunicationType == EDMXCommunicationType::InternalOnly)
		{
			// Skip internal only, it is not an option that is exposed to users
			continue;
		}

		const TSharedPtr<FString>& CommunicationTypeString = CommunicationTypeToStringMap.FindChecked(CommunicationType);
		
		CommunicationTypesSource.Add(CommunicationTypeString);
	}
}

TSharedRef<SWidget> SDMXCommunicationTypeComboBox::GenerateCommunicationTypeComboBoxEntry(TSharedPtr<FString> InCommunicationType)
{
	return
		SNew(STextBlock)
		.Text(FText::FromString(*InCommunicationType))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

void SDMXCommunicationTypeComboBox::HandleCommunicationTypeSelectionChanged(TSharedPtr<FString> InCommunicationType, ESelectInfo::Type InSelectInfo)
{
	check(CommunicationTypeTextBlock.IsValid());

	if (ensure(InCommunicationType.IsValid()))
	{
		CommunicationTypeTextBlock->SetText(FText::FromString(*InCommunicationType));

		if (InSelectInfo != ESelectInfo::Direct)
		{
			OnCommunicationTypeSelected.ExecuteIfBound();
		}
	}
}
