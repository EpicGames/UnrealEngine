// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

template<typename OptionType> class SComboBox;
class STextBlock;


/** 
 * Helper widget to select a Communication Type.
 *
 * Can adjust the types via SetCommunicationTypes after construction.
 * Hides itself if no communication type is available.
 */
class SDMXCommunicationTypeComboBox
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXCommunicationTypeComboBox)
		:  _InitialCommunicationType(EDMXCommunicationType::InternalOnly)
		{}

		SLATE_ARGUMENT(TArray<EDMXCommunicationType>, CommunicationTypes)

		SLATE_ARGUMENT(EDMXCommunicationType, InitialCommunicationType)

		SLATE_EVENT(FSimpleDelegate, OnCommunicationTypeSelected)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Returns the selected Communication Type */
	EDMXCommunicationType GetSelectedCommunicationType() const;

private:
	/** Sets the communication type but doesn't raise an external event, useful for initialization */
	void SetCommunicationTypesInternal(const TArray<EDMXCommunicationType>& NewCommunicationTypes);

	/** Generates the CommunicationTypesSource from the provided Communication Types */
	void GenerateOptionsSource(const TArray<EDMXCommunicationType>& CommunicationTypes);

	/** Called when the combo box creates its entries */
	TSharedRef<SWidget> GenerateCommunicationTypeComboBoxEntry(TSharedPtr<FString> InCommunicationTypeString);

	/** Called when the communication type in the combo box changed */
	void HandleCommunicationTypeSelectionChanged(TSharedPtr<FString> InCommunicationTypeString, ESelectInfo::Type InSelectInfo);

	/** The actual combo box */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> CommunicationTypeComboBox;

	/** A string for each communication type to ease conversion */
	static const TMap<EDMXCommunicationType, TSharedPtr<FString>> CommunicationTypeToStringMap;

	/** Array of Communication Types to serve as combo box source */
	TArray<TSharedPtr<FString>> CommunicationTypesSource;

	/** Text box shown on top of the Communication Type combo box */
	TSharedPtr<STextBlock> CommunicationTypeTextBlock;

	/** Delegate executed when a Communication Type was selected */
	FSimpleDelegate OnCommunicationTypeSelected;
};
