// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXDelayEditWidget.h"

#include "DMXProtocolSettings.h"

#include "CommonFrameRates.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SFrameRateEntryBox.h"
#include "Widgets/Input/SEditableTextBox.h"


#define LOCTEXT_NAMESPACE "SDMXDelayEditWidget"

void SDMXDelayEditWidget::Construct(const FArguments& InArgs)
{
	OnDelayChanged = InArgs._OnDelayChanged;
	OnDelayFrameRateChanged = InArgs._OnDelayFrameRateChanged;

	Delay = InArgs._InitialDelay;
	DelayFrameRate = InArgs._InitialDelayFrameRate;

	const FText InitialDelayText = FText::FromString(FString::SanitizeFloat(Delay));
	
	const TAttribute<EVisibility> FPSVisiblityAttribute = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this]()
		{
			return DelayFrameRate.AsDecimal() == 1.f ? EVisibility::Collapsed : EVisibility::Visible;
		}));


	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SAssignNew(ValueEditBox, SEditableTextBox)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(InitialDelayText)
			.OnVerifyTextChanged(this, &SDMXDelayEditWidget::OnVerifyValueTextChanged)
			.OnTextCommitted(this, &SDMXDelayEditWidget::OnValueTextCommitted)
		]
		
	+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(8.f, 0.f, 8.f, 0.f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text_Lambda([this]
				{
					if (Delay == 1.0)
					{
						return LOCTEXT("FramesPerSecondLabelSingular", "frame at");
					}
					else
					{
						return LOCTEXT("FramesPerSecondLabelPlural", "frames at");
					}
				})
			.Visibility(FPSVisiblityAttribute)
		]

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &SDMXDelayEditWidget::GenerateDelayTypeMenu)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text_Lambda([this]()
					{
						const double DelayFrameRateDecimal = DelayFrameRate.AsDecimal();
						if (DelayFrameRateDecimal == 1.0)
						{
							return LOCTEXT("SecondsLabel", "Seconds");
						}
						else
						{
							return FText::Format(LOCTEXT("FramesPerSecond", "{0} Frames Per Second"), DelayFrameRateDecimal);
						}
					})
			]
		]
	];
}

double SDMXDelayEditWidget::GetDelay() const
{
	return Delay;
}

FFrameRate SDMXDelayEditWidget::GetDelayFrameRate() const
{
	return DelayFrameRate;
}

TSharedRef<SWidget> SDMXDelayEditWidget::GenerateDelayTypeMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	const FFrameRate OneFramePerSecond = FFrameRate(1, 1);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("Seconds", "As Seconds"),
		LOCTEXT("SecondsDescription", "Sets the delay to Seconds"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SDMXDelayEditWidget::OnDelayTypeChanged, OneFramePerSecond))
		);

	MenuBuilder.AddMenuSeparator();

	TArray<FCommonFrameRateInfo> CommonFrameRates;
	for (const FCommonFrameRateInfo& Info : FCommonFrameRates::GetAll())
	{
		CommonFrameRates.Add(Info);
	}

	CommonFrameRates.Sort(
		[](const FCommonFrameRateInfo& A, const FCommonFrameRateInfo& B)
		{
			return A.FrameRate.AsDecimal() < B.FrameRate.AsDecimal();
		}
	);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("RecommendedRates", "As Frame Rate"));
	{
		for (const FCommonFrameRateInfo& Info : CommonFrameRates)
		{
			MenuBuilder.AddMenuEntry(
				Info.DisplayName,
				Info.Description,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SDMXDelayEditWidget::OnDelayTypeChanged, Info.FrameRate),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SDMXDelayEditWidget::IsSameDelayType, Info.FrameRate)
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}

		MenuBuilder.AddWidget(
			SNew(SBox)
			.HAlign(HAlign_Right)
			.MaxDesiredWidth(100.f)
			[
				SNew(SFrameRateEntryBox)
				.Value_Lambda([this]
					{
						return DelayFrameRate;
					})
				.OnValueChanged(this, &SDMXDelayEditWidget::OnDelayTypeChanged)
			],
			LOCTEXT("CustomFramerateDisplayLabel", "Custom")
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDMXDelayEditWidget::OnDelayTypeChanged(FFrameRate NewDelayFrameRate)
{
	DelayFrameRate = NewDelayFrameRate;

	OnDelayFrameRateChanged.ExecuteIfBound();
}

bool SDMXDelayEditWidget::IsSameDelayType(FFrameRate FrameRate) const
{
	if (DelayFrameRate == FrameRate)
	{
		return true;
	}

	return false;
}

bool SDMXDelayEditWidget::OnVerifyValueTextChanged(const FText& InNewText, FText& OutErrorMessage)
{
	FString StringValue = InNewText.ToString();
	int32 Value;
	if (LexTryParseString<int32>(Value, *StringValue))
	{
		if (Value >= 0.0)
		{
			OutErrorMessage = FText::GetEmpty();
			return true;
		}
	}

	OutErrorMessage = LOCTEXT("InvalidValueError", "Only numeric values > 0 are supported");
	return false;
}

void SDMXDelayEditWidget::OnValueTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	FString StringValue = InNewText.ToString();
	double Value;
	if (LexTryParseString<double>(Value, *StringValue))
	{
		if (Value >= 0.0)
		{
			Delay = Value;

			const FText ValueText = FText::FromString(FString::SanitizeFloat(Delay));
			ValueEditBox->SetText(ValueText);

			OnDelayChanged.ExecuteIfBound();
		}
	}
}

bool SDMXDelayEditWidget::DisplayAsFramesPerSecond() const
{
	const double Value = DelayFrameRate.AsDecimal() / Delay;

	return FMath::IsNearlyEqual(Value, 1.0);
}

#undef LOCTEXT_NAMESPACE
