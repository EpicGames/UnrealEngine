// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Misc/FrameRate.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

class SEditableTextBox;


/** Helper widget to select a delay either by Seconds or by a Frame Rate */
class SDMXDelayEditWidget
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXDelayEditWidget)
		{}

		/** The initially displayed delay in seconds */
		SLATE_ARGUMENT(double, InitialDelay)

		/** The initially selected frame rate (1.0 for seconds) */
		SLATE_ARGUMENT(FFrameRate, InitialDelayFrameRate)

		/** Delegate executed when the delay seconds changed */
		SLATE_EVENT(FSimpleDelegate, OnDelayChanged)

		/** Delegate executed when the delay frame rate changed */
		SLATE_EVENT(FSimpleDelegate, OnDelayFrameRateChanged)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Returns the delay in seconds */
	double GetDelay() const;

	/** Returns the delay frame rate */
	FFrameRate GetDelayFrameRate() const;

private:
	/** Generates the menu to select a Type */
	TSharedRef<SWidget> GenerateDelayTypeMenu();

	/** Called when the value text changed but was not commited */
	bool OnVerifyValueTextChanged(const FText& InNewText, FText& OutErrorMessage);

	/** Called when the value text was commited */
	void OnValueTextCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	/** Called when the delay Type changed  */
	void OnDelayTypeChanged(FFrameRate NewDelayFrameRate);

	/** Called to test if the delay Type is the same as the selected one */
	bool IsSameDelayType(FFrameRate FrameRate) const;

	/** Returns true if values should be displayed as frames per second */
	bool DisplayAsFramesPerSecond() const;

	/** Box to enter the Delay value */
	TSharedPtr<SEditableTextBox> ValueEditBox;

	/** Delegate executed when the selected delay changed */
	FSimpleDelegate OnDelayChanged;

	/** Delegate executed when the selected delay changed */
	FSimpleDelegate OnDelayFrameRateChanged;

	/** The currently set delay, in seconds */
	double Delay = 0.0;

	/** The frame rate in which the delay is displayed */
	FFrameRate DelayFrameRate;
};
