// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXIPAddressEditWidget.h"

#include "DMXProtocolModule.h"
#include "DMXProtocolSettings.h"
#include "DMXProtocolUtils.h"

#include "Styling/AppStyle.h"
#include "IPAddress.h"
#include "SocketSubsystem.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Input/SEditableTextBox.h"


void SDMXIPAddressEditWidget::Construct(const FArguments& InArgs)
{
	OnIPAddressSelectedDelegate = InArgs._OnIPAddressSelected;

	LocalAdapterAddressSource = FDMXProtocolUtils::GetLocalNetworkInterfaceCardIPs();

	const TSharedRef<SEditableTextBox> IPAddressEditTextBox = SAssignNew(IPAddressEditableTextBlock, SEditableTextBox)
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.Text(FText::FromString(InArgs._InitialValue))
		.OnTextCommitted(this, &SDMXIPAddressEditWidget::OnIPAddressTextCommmited);

	if (InArgs._bShowLocalNICComboBox)
	{
		ChildSlot
		[
			SAssignNew(LocalAdapterAddressComboBox, SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&LocalAdapterAddressSource)
			.OnGenerateWidget(this, &SDMXIPAddressEditWidget::GenerateLocalAdapterAddressComboBoxEntry)
			.OnSelectionChanged(this, &SDMXIPAddressEditWidget::OnIpAddressSelected)
			.Content()
			[
				IPAddressEditTextBox
			]
		];
	}
	else
	{
		ChildSlot
		[
			IPAddressEditTextBox
		];
	}
}

FString SDMXIPAddressEditWidget::GetSelectedIPAddress() const
{
	return IPAddressEditableTextBlock->GetText().ToString();
}

void SDMXIPAddressEditWidget::OnIpAddressSelected(TSharedPtr<FString> InAddress, ESelectInfo::Type InType)
{
	TSharedPtr<FString> SelectedIPAddress = LocalAdapterAddressComboBox->GetSelectedItem();
	if (SelectedIPAddress.IsValid())
	{
		IPAddressEditableTextBlock->SetText(FText::FromString(*SelectedIPAddress));
		OnIPAddressSelectedDelegate.ExecuteIfBound();
	}
}

void SDMXIPAddressEditWidget::OnIPAddressTextCommmited(const FText&, ETextCommit::Type)
{
	OnIPAddressSelectedDelegate.ExecuteIfBound();
}

TSharedRef<SWidget> SDMXIPAddressEditWidget::GenerateLocalAdapterAddressComboBoxEntry(TSharedPtr<FString> InAddress)
{
	return
		SNew(STextBlock)
		.Text(FText::FromString(*InAddress))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}
