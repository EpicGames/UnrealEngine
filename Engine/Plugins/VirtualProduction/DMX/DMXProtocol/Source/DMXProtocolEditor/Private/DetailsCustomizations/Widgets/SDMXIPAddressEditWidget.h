// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

class SEditableTextBox;
class SInlineEditableTextBlock;
class SWidgetSwitcher;


/**
 * Helper widget that draws a local IP address in a combobox.
 */
class SDMXIPAddressEditWidget
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXIPAddressEditWidget)
		: _bShowLocalNICComboBox(false)
		{}

		SLATE_ARGUMENT(FString, InitialValue)

		SLATE_ARGUMENT(bool, bShowLocalNICComboBox)

		SLATE_EVENT(FSimpleDelegate, OnIPAddressSelected)

	SLATE_END_ARGS()


	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Returns the selected IP Address */
	FString GetSelectedIPAddress() const;

private:
	/** Handles changes in the local adapter address combo box */
	void OnIpAddressSelected(TSharedPtr<FString> InAddress, ESelectInfo::Type InType);

	/** Called when the IP Address was commited in the editable text block */
	void OnIPAddressTextCommmited(const FText&, ETextCommit::Type);

	/** Generates an entry in the local adapter address combo box */
	TSharedRef<SWidget> GenerateLocalAdapterAddressComboBoxEntry(TSharedPtr<FString> InAddress);

	/** ComboBox shown when in local adapter mode */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> LocalAdapterAddressComboBox;

	/** Delegate executed when a local IP address was selected */
	FSimpleDelegate OnIPAddressSelectedDelegate;

	/** Array of LocalAdapterAddresses available to the system */
	TArray<TSharedPtr<FString>> LocalAdapterAddressSource;

	/** Text box shown on top of the local adapter address address combo box */
	TSharedPtr<SEditableTextBox> IPAddressEditableTextBlock;
};

