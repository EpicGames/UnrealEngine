// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXProtocolNameComboBox.h"

#include "DMXProtocolModule.h"

#include "Styling/AppStyle.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Text/STextBlock.h" 


void SDMXProtocolNameComboBox::Construct(const FArguments& InArgs)
{
	OnProtocolNameSelected = InArgs._OnProtocolNameSelected;

	// Cache available protocols as source for the combo box, and for later lookup in a map
	FDMXProtocolModule& DMXProtocolModule = FModuleManager::GetModuleChecked<FDMXProtocolModule>("DMXProtocol");
	const TMap<FName, IDMXProtocolFactory*>& Protocols = DMXProtocolModule.GetProtocolFactories();
	
	TArray<FName> ProtocolNames;
	Protocols.GenerateKeyArray(ProtocolNames);

	TSharedPtr<FString> InitiallySelectedItem;
	for (const FName& ProtocolName : ProtocolNames)
	{
		TSharedRef<FString> ProtocolNameString = MakeShared<FString>(ProtocolName.ToString());
		ProtocolNamesSource.Add(ProtocolNameString);
		StringToProtocolMap.Add(ProtocolNameString, ProtocolName);

		if (ProtocolName == InArgs._InitiallySelectedProtocolName)
		{
			InitiallySelectedItem = ProtocolNameString;
		}
	}
	
	if (!InitiallySelectedItem.IsValid())
	{
		if (ProtocolNamesSource.Num() > 0)
		{
			InitiallySelectedItem = ProtocolNamesSource[0];
		}
		else
		{
			ProtocolNamesSource.Add(MakeShared<FString>(TEXT("No protocols available")));
			InitiallySelectedItem = ProtocolNamesSource[0];
		}
	}
	check(InitiallySelectedItem.IsValid());

	ChildSlot
	[
		SAssignNew(ProtocolNameComboBox, SComboBox<TSharedRef<FString>>)
		.OptionsSource(&ProtocolNamesSource)
		.InitiallySelectedItem(InitiallySelectedItem)
		.OnGenerateWidget(this, &SDMXProtocolNameComboBox::GenerateProtocolNameComboBoxEntry)
		.OnSelectionChanged(this, &SDMXProtocolNameComboBox::HandleProtocolNameSelectionChanged)
		.Content()
		[
			SAssignNew(ProtocolNameTextBlock, STextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(FText::FromString(*InitiallySelectedItem))
		]
	];
}

FName SDMXProtocolNameComboBox::GetSelectedProtocolName() const
{
	check(ProtocolNameComboBox.IsValid());

	TSharedPtr<FString> SelectedItem = ProtocolNameComboBox->GetSelectedItem();
	check(SelectedItem.IsValid());

	return StringToProtocolMap.FindChecked(SelectedItem.ToSharedRef());
}

TSharedRef<SWidget> SDMXProtocolNameComboBox::GenerateProtocolNameComboBoxEntry(TSharedRef<FString> InProtocolNameString)
{
	return
		SNew(STextBlock)
		.Text(FText::FromString(*InProtocolNameString))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

void SDMXProtocolNameComboBox::HandleProtocolNameSelectionChanged(TSharedPtr<FString> InProtocolNameString, ESelectInfo::Type InSelectInfo)
{
	check(ProtocolNameTextBlock.IsValid());

	ProtocolNameTextBlock->SetText(FText::FromString(*InProtocolNameString));

	OnProtocolNameSelected.ExecuteIfBound();
}

