// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

class STextBlock;


/** Helper widget to select a Protocol Name */
class SDMXProtocolNameComboBox
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXProtocolNameComboBox)
		{}

		SLATE_ARGUMENT(FName, InitiallySelectedProtocolName)

		SLATE_EVENT(FSimpleDelegate, OnProtocolNameSelected)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Returns the selected Communication Type */
	FName GetSelectedProtocolName() const;

private:
	TSharedRef<SWidget> GenerateProtocolNameComboBoxEntry(TSharedRef<FString> InProtocolNameString);

	void HandleProtocolNameSelectionChanged(TSharedPtr<FString> InProtocolNameString, ESelectInfo::Type InSelectInfo);

	TSharedPtr<SComboBox<TSharedRef<FString>>> ProtocolNameComboBox;

	/** Array of Communication Types to serve as combo box source */
	TArray<TSharedRef<FString>> ProtocolNamesSource;

	/** Protocol Names with their string repesentation in the combo box */
	TMap<TSharedRef<FString>, FName> StringToProtocolMap;

	/** Text box shown on top of the Communication Type combo box */
	TSharedPtr<STextBlock> ProtocolNameTextBlock;

	/** Delegate executed when a Communication Type was selected */
	FSimpleDelegate OnProtocolNameSelected;
};

