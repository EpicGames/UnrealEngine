// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXPortSelector.h"

#include "DMXProtocolSettings.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXPortManager.h"

#include "Styling/AppStyle.h"


#define LOCTEXT_NAMESPACE "FDMXPortSelectorItem"

TSharedRef<FDMXPortSelectorItem> FDMXPortSelectorItem::CreateTitleRowItem(const FString& TitleString)
{
	TSharedRef<FDMXPortSelectorItem> NewItem = MakeShared<FDMXPortSelectorItem>();

	NewItem->ItemName = TitleString;
	NewItem->Type = EDMXPortSelectorItemType::TitleRow;

	return NewItem;
}

TSharedRef<FDMXPortSelectorItem> FDMXPortSelectorItem::CreateItem(const FDMXInputPortConfig& InInputPortConfig)
{
	TSharedRef<FDMXPortSelectorItem> NewItem = MakeShared<FDMXPortSelectorItem>();

	NewItem->ItemName = InInputPortConfig.GetPortName();
	NewItem->PortGuid = InInputPortConfig.GetPortGuid();
	NewItem->Type = EDMXPortSelectorItemType::Input;

	check(NewItem->PortGuid.IsValid());
	return NewItem;
}

TSharedRef<FDMXPortSelectorItem> FDMXPortSelectorItem::CreateItem(const FDMXOutputPortConfig& InOutputPortConfig)
{
	TSharedRef<FDMXPortSelectorItem> NewItem = MakeShared<FDMXPortSelectorItem>();

	NewItem->ItemName = InOutputPortConfig.GetPortName();
	NewItem->PortGuid = InOutputPortConfig.GetPortGuid();
	NewItem->Type = EDMXPortSelectorItemType::Output;

	check(NewItem->PortGuid.IsValid());
	return NewItem;
}

void SDMXPortSelector::Construct(const FArguments& InArgs)
{
	Mode = InArgs._Mode;
	OnPortSelected = InArgs._OnPortSelected;

	// Bind to editor property changes
	UDMXProtocolSettings* ProtocolSettings = GetMutableDefault<UDMXProtocolSettings>();
	check(ProtocolSettings);

	FDMXPortManager::Get().OnPortsChanged.AddSP(this, &SDMXPortSelector::OnPortsChanged);

	ChildSlot
		[
			SAssignNew(ContentBorder, SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
		];

	GenerateWidgetsFromPorts();

	TSharedPtr<FDMXPortSelectorItem> InitialSelection =	FindPortItem(InArgs._InitialSelection);
	if (InitialSelection.IsValid())
	{
		check(PortNameComboBox.IsValid());
		PortNameComboBox->SetSelectedItem(InitialSelection);
	}

	bIsUnderConstruction = false;
}

bool SDMXPortSelector::IsInputPortSelected() const
{
	if (!bHasMultipleValues)
	{
		const TSharedPtr<FDMXPortSelectorItem> CurrentSelection = PortNameComboBox->GetSelectedItem();
		if (CurrentSelection.IsValid())
		{
			return CurrentSelection->GetType() == EDMXPortSelectorItemType::Input;
		}
	}

	return false;
}

bool SDMXPortSelector::IsOutputPortSelected() const
{
	if (!bHasMultipleValues)
	{
		const TSharedPtr<FDMXPortSelectorItem> CurrentSelection = PortNameComboBox->GetSelectedItem();
		if (CurrentSelection.IsValid())
		{
			return CurrentSelection->GetType() == EDMXPortSelectorItemType::Output;
		}
	}

	return false;
}

FDMXInputPortSharedPtr SDMXPortSelector::GetSelectedInputPort() const
{
	if (!bHasMultipleValues)
	{
		const TSharedPtr<FDMXPortSelectorItem> CurrentSelection = PortNameComboBox->GetSelectedItem();
		if (CurrentSelection.IsValid() &&
			CurrentSelection->GetType() == EDMXPortSelectorItemType::Input)
		{
			const TArray<FDMXInputPortSharedRef>& InputPorts = FDMXPortManager::Get().GetInputPorts();

			const FDMXInputPortSharedRef* SelectedPortPtr = InputPorts.FindByPredicate([&CurrentSelection](const FDMXInputPortSharedRef& InputPort) {
				return InputPort->GetPortGuid() == CurrentSelection->GetGuid();
				});

			if (SelectedPortPtr)
			{
				return *SelectedPortPtr;
			}
		}
	}

	return nullptr;
}

FDMXOutputPortSharedPtr SDMXPortSelector::GetSelectedOutputPort() const
{
	if (!bHasMultipleValues)
	{
		const TSharedPtr<FDMXPortSelectorItem> CurrentSelection = PortNameComboBox->GetSelectedItem();
		if (CurrentSelection.IsValid() &&
			CurrentSelection->GetType() == EDMXPortSelectorItemType::Output)
		{
			const TArray<FDMXOutputPortSharedRef>& OutputPorts = FDMXPortManager::Get().GetOutputPorts();

			const FDMXOutputPortSharedRef* SelectedPortPtr = OutputPorts.FindByPredicate([&CurrentSelection](const FDMXOutputPortSharedRef& OutputPort) {
				return OutputPort->GetPortGuid() == CurrentSelection->GetGuid();
				});

			if (SelectedPortPtr)
			{
				return *SelectedPortPtr;
			}
		}
	}

	return nullptr;
}

bool SDMXPortSelector::HasSelection() const
{
	return PortNameComboBox->GetSelectedItem().IsValid();
}

void SDMXPortSelector::SelectPort(const FGuid& PortGuid)
{
	bHasMultipleValues = false;

	TSharedPtr<FDMXPortSelectorItem> PortItem = FindPortItem(PortGuid);
	if (PortItem != PortNameComboBox->GetSelectedItem())
	{
		if (PortItem.IsValid())
		{
			PortNameComboBox->SetSelectedItem(PortItem);
		}
		else
		{
			PortNameComboBox->ClearSelection();
		}

		OnPortSelected.ExecuteIfBound();
	}
}

void SDMXPortSelector::SetHasMultipleValues()
{
	bHasMultipleValues = true;

	PortNameTextBlock->SetText(LOCTEXT("ShowAsMultipleValues", "Mutiple Values"));
}

TSharedRef<SWidget> SDMXPortSelector::GenerateComboBoxEntry(TSharedPtr<FDMXPortSelectorItem> Item)
{
	if (Item->IsTitleRow())
	{
		return
			SNew(STextBlock)
			.Text(FText::FromString(*Item->GetItemName()))
			.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"));
	}

	return
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(4.f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(*Item->GetItemName()))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		];
}

void SDMXPortSelector::OnPortItemSelectionChanged(TSharedPtr<FDMXPortSelectorItem> Item, ESelectInfo::Type InSelectInfo)
{
	check(PortNameComboBox.IsValid());
	check(PortNameTextBlock.IsValid());
	
	if (Item.IsValid())
	{
		bHasMultipleValues = false;

		if (Item->IsTitleRow())
		{
			// Restore the previous selection if possible
			if (RestoreItem.IsValid())
			{
				PortNameComboBox->SetSelectedItem(RestoreItem);
			}
		}
		else
		{
			if (Item.IsValid())
			{
				RestoreItem = Item;
				PortNameTextBlock->SetText(FText::FromString(Item->GetItemName()));
			}
			else
			{
				PortNameTextBlock->SetText(LOCTEXT("SelectPortPrompt", "Please select a port"));
			}

			// Only broadcast changes after construction
			if (!bIsUnderConstruction)
			{
				OnPortSelected.ExecuteIfBound();
			}
		}
	}
}

TArray<TSharedPtr<FDMXPortSelectorItem>> SDMXPortSelector::MakeComboBoxSource()
{
	TArray<TSharedPtr<FDMXPortSelectorItem>> NewComboBoxSource;

	const UDMXProtocolSettings* ProtocolSettings = GetDefault<UDMXProtocolSettings>();
	check(ProtocolSettings);

	if (Mode == EDMXPortSelectorMode::SelectFromAvailableInputs ||
		Mode == EDMXPortSelectorMode::SelectFromAvailableInputsAndOutputs)
	{
		NewComboBoxSource.Add(FDMXPortSelectorItem::CreateTitleRowItem(TEXT("Inputs")));

		for (const FDMXInputPortConfig& InputPortConfig : ProtocolSettings->InputPortConfigs)
		{
			// Only show ports with valid guids. Let the config log and mend invalid Guids and ignore them quietly here.
			if (InputPortConfig.GetPortGuid().IsValid())
			{
			TSharedRef<FDMXPortSelectorItem> Item = FDMXPortSelectorItem::CreateItem(InputPortConfig);

			NewComboBoxSource.Add(Item);
		}
	}
	}

	if (Mode == EDMXPortSelectorMode::SelectFromAvailableOutputs ||
		Mode == EDMXPortSelectorMode::SelectFromAvailableInputsAndOutputs)
	{
		NewComboBoxSource.Add(FDMXPortSelectorItem::CreateTitleRowItem(TEXT("Outputs")));

		for (const FDMXOutputPortConfig& OutputPortConfig : ProtocolSettings->OutputPortConfigs)
		{
			// Only show ports with valid guids. Let the config log and mend invalid Guids and ignore them quietly here.
			if (OutputPortConfig.GetPortGuid().IsValid())
			{
			TSharedRef<FDMXPortSelectorItem> Item = FDMXPortSelectorItem::CreateItem(OutputPortConfig);

			NewComboBoxSource.Add(Item);
		}
	}
	}

	return NewComboBoxSource;
}

void SDMXPortSelector::GenerateWidgetsFromPorts()
{
	const FGuid SelectedGuid = [this]()
	{
		if (PortNameComboBox.IsValid())
		{
			if (TSharedPtr<FDMXPortSelectorItem> Item = PortNameComboBox->GetSelectedItem())
			{
				return Item->GetGuid();
			}
		}
		return FGuid();
	}();

	ComboBoxSource = MakeComboBoxSource(); 
	if (ComboBoxSource.Num() > 0)
	{
		ContentBorder->SetContent(
			SNew(SBox)
			[
				SAssignNew(PortNameComboBox, SComboBox<TSharedPtr<FDMXPortSelectorItem>>)
				.OptionsSource(&ComboBoxSource)
				.OnGenerateWidget(this, &SDMXPortSelector::GenerateComboBoxEntry)
				.OnSelectionChanged(this, &SDMXPortSelector::OnPortItemSelectionChanged)
				.Content()
				[
					SAssignNew(PortNameTextBlock, STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]
			]);

		PortNameComboBox->RefreshOptions();

		// Try to restore the selection. Will trigger OnPortItemSelectionChanged
		const TSharedPtr<FDMXPortSelectorItem>* SelectedPortItemPtr = ComboBoxSource.FindByPredicate([SelectedGuid](const TSharedPtr<FDMXPortSelectorItem>& Item)
			{
				return Item->GetGuid() == SelectedGuid;
			});

		if (SelectedPortItemPtr)
		{
			PortNameComboBox->SetSelectedItem(*SelectedPortItemPtr);
		}
		else if (TSharedPtr<FDMXPortSelectorItem> NewSelection = GetFirstPortInComboBoxSource())
		{
			// We know there is an entry in combo box source, a valid NewSelection is to be expected
			check(NewSelection.IsValid()); 

			// Set the selected item, this will trigger triggers OnPortItemSelectionChanged
			PortNameComboBox->SetSelectedItem(NewSelection);
		}
	}
	else
	{
		ContentBorder->SetContent(
			SNew(SBox)
			[
				SAssignNew(PortNameComboBox, SComboBox<TSharedPtr<FDMXPortSelectorItem>>)
				.OptionsSource(&ComboBoxSource)
				.Content()
				[
					SAssignNew(PortNameTextBlock, STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Text(LOCTEXT("NoPortsDefinedInProjectSettings", "No DMX ports defined in Project Settings"))
				]
			]);

		// Clear the selection. Will trigger OnPortItemSelectionChanged
		PortNameComboBox->ClearSelection();
	}
}

TSharedPtr<FDMXPortSelectorItem> SDMXPortSelector::FindPortItem(const FGuid& PortGuid) const
{
	// Make an intial selection from the DesiredGuid
	if (PortGuid.IsValid())
	{
		for (const TSharedPtr<FDMXPortSelectorItem>& Item : ComboBoxSource)
		{
			if (Item->GetGuid() == PortGuid)
			{
				return Item;
			}
		}
	}

	return nullptr;
}

TSharedPtr<FDMXPortSelectorItem> SDMXPortSelector::GetFirstPortInComboBoxSource() const
{
	const TSharedPtr<FDMXPortSelectorItem>* PortItemPtr = ComboBoxSource.FindByPredicate([](const TSharedPtr<FDMXPortSelectorItem>& Item) {
		return Item->GetType() != EDMXPortSelectorItemType::TitleRow;
		});


	return PortItemPtr ? *PortItemPtr : nullptr;
}

void SDMXPortSelector::OnPortsChanged()
{
	check(PortNameComboBox.IsValid());

	GenerateWidgetsFromPorts();
}

#undef LOCTEXT_NAMESPACE
