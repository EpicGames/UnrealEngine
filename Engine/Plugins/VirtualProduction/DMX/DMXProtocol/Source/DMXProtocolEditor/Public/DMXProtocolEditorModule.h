// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/** Implements the DMXProtocolEditor Module  */
class DMXPROTOCOLEDITOR_API FDMXProtocolEditorModule 
	: public IModuleInterface
{
public:
	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface implementation

	/** Get the instance of this module. */
	static FDMXProtocolEditorModule& Get();

private:
	/** Registers details customizations */
	void RegisterDetailsCustomizations();

	/** Unregisters details customizations */
	void UnregisterDetailsCustomizations();
};
