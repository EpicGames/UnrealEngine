// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

#include "Misc/Guid.h" 
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

struct FDMXInputPortConfig;
struct FDMXOutputPortConfig;
class FDMXInputPort;

class SBorder;


enum class EDMXPortSelectorMode : uint8
{
	SelectFromAvailableInputs,
	SelectFromAvailableOutputs,
	SelectFromAvailableInputsAndOutputs
};

enum class EDMXPortSelectorItemType : uint8
{
	TitleRow,
	Input,
	Output
};

/** An item in the Port Selector Combo Box */
class FDMXPortSelectorItem
	: public TSharedFromThis<FDMXPortSelectorItem>
{
public:
	static TSharedRef<FDMXPortSelectorItem> CreateTitleRowItem(const FString& TitleString);

	static TSharedRef<FDMXPortSelectorItem> CreateItem(const FDMXInputPortConfig& InPortConfig);

	static TSharedRef<FDMXPortSelectorItem> CreateItem(const FDMXOutputPortConfig& InPortConfig);

	bool IsTitleRow() const { return !PortGuid.IsValid(); }

	const FGuid& GetGuid() const { return PortGuid; }

	const FString& GetItemName() const { return ItemName; }

	const EDMXPortSelectorItemType GetType() const { return Type; }

private:
	FString ItemName;

	/** Port Guid, invalid if a title row */
	FGuid PortGuid;

	/** The type of the item */
	EDMXPortSelectorItemType Type;
};

/** 
 * A widget that allows to select a DMX port available in from the DMXPortManager.
 * 
 * Directly returns corresponding DMXPort via the GetPort method.
 */
class DMXPROTOCOLEDITOR_API SDMXPortSelector
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXPortSelector)
		: _Mode(EDMXPortSelectorMode::SelectFromAvailableInputsAndOutputs)
		, _InitialSelection(FGuid())
	{}

		SLATE_ARGUMENT(EDMXPortSelectorMode, Mode)

		SLATE_ARGUMENT(FGuid, InitialSelection)

		SLATE_EVENT(FSimpleDelegate, OnPortSelected)

	SLATE_END_ARGS()

	SDMXPortSelector()
		: bIsUnderConstruction(true)
	{}

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Returns true if an input port is selected */
	bool IsInputPortSelected() const;

	/** Returns true if an output port is selected */
	bool IsOutputPortSelected() const;

	/** Returns the selected input port, or nullptr if no valid input port is selected  */
	FDMXInputPortSharedPtr GetSelectedInputPort() const;

	/** Returns the selected output port, or nullptr if no valid output port is selected  */
	FDMXOutputPortSharedPtr GetSelectedOutputPort() const;

	/** Returns true if a valid port is selected */
	bool HasSelection() const;
	
	/** Selects specified Port by Port Guid. If the Guid is not valid, clears the selection */
	void SelectPort(const FGuid& PortGuid);

	/** 
	 * Instead of showing a selected port, the widget displays 'Multiple Values' and no longer returns a selected port.
	 * The widget will no longer display 'Multiple Values' when a selection was made. 
	 */
	void SetHasMultipleValues();

private:
	/** Called to generate entries in the port name combo box */
	TSharedRef<SWidget> GenerateComboBoxEntry(TSharedPtr<FDMXPortSelectorItem> Item);
	
	/** Called when an item was selected in the port name combo box */
	void OnPortItemSelectionChanged(TSharedPtr<FDMXPortSelectorItem> Item, ESelectInfo::Type InSelectInfo);

	/** Creates a a combo box source froma what's currently in settings */
	TArray<TSharedPtr<FDMXPortSelectorItem>> MakeComboBoxSource();

	/** Generates new widgets in the selector from currently available port configs */
	void GenerateWidgetsFromPorts();

	/** Returns the port if it exists, or nullptr if the port doesn't exist */
	TSharedPtr<FDMXPortSelectorItem> FindPortItem(const FGuid& PortGuid) const;

	/** Returns the first valid port from the combo box source, or nullptr if there is no valid port */
	TSharedPtr<FDMXPortSelectorItem> GetFirstPortInComboBoxSource() const;

	/** Called when ports changed */
	void OnPortsChanged();

	/** State where the widget was set to have multiple values */
	bool bHasMultipleValues = false;

	/** The combo box that holds all ports */
	TSharedPtr<SComboBox<TSharedPtr<FDMXPortSelectorItem>>> PortNameComboBox;

	/** Array of Port Selector Items to serve as combo box source */
	TArray<TSharedPtr<FDMXPortSelectorItem>> ComboBoxSource;

	/** The last selected item. Use only to restore from selecting a category row. */
	TSharedPtr<FDMXPortSelectorItem> RestoreItem;

	/** Text box shown on top of the Port Name Combo Box */
	TSharedPtr<STextBlock> PortNameTextBlock;

	/** Border that holds all the arbitrary content, to switch content dynamically*/
	TSharedPtr<SBorder> ContentBorder;
	
	/** Defines which ports are shown */
	EDMXPortSelectorMode Mode;

	/** True during Construct, to avoid raising OnPortSelected while the widget is not valid yet */
	bool bIsUnderConstruction;

	FSimpleDelegate OnPortSelected;
};
