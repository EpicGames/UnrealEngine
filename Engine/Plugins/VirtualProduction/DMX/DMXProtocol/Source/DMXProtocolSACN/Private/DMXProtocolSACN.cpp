// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolSACN.h"

#include "DMXProtocolSACNConstants.h"
#include "DMXProtocolSACNReceiver.h"
#include "DMXProtocolSACNSender.h"
#include "DMXStats.h"
#include "IO/DMXInputPort.h"
#include "IO/DMXOutputPort.h"

#include "SocketSubsystem.h"


DECLARE_CYCLE_STAT(TEXT("SACN Packages Enqueue To Send"), STAT_SACNPackagesEnqueueToSend, STATGROUP_DMX);

const TArray<EDMXCommunicationType> FDMXProtocolSACN::InputPortCommunicationTypes = TArray<EDMXCommunicationType>(
	{ 
		EDMXCommunicationType::InternalOnly 
	});

const TArray<EDMXCommunicationType> FDMXProtocolSACN::OutputPortCommunicationTypes = TArray<EDMXCommunicationType>(
	{ 
		EDMXCommunicationType::Multicast, 
		EDMXCommunicationType::Unicast 
	});

FDMXProtocolSACN::FDMXProtocolSACN(const FName& InProtocolName)
	: ProtocolName(InProtocolName)
{}

bool FDMXProtocolSACN::Init()
{
	return true;
}

bool FDMXProtocolSACN::Shutdown()
{
	return true;
}

bool FDMXProtocolSACN::Tick(float DeltaTime)
{
	checkNoEntry();
	return false;
}

bool FDMXProtocolSACN::IsEnabled() const
{
	return true;
}

const FName& FDMXProtocolSACN::GetProtocolName() const
{
	return ProtocolName;
}

const TArray<EDMXCommunicationType> FDMXProtocolSACN::GetInputPortCommunicationTypes() const
{
	return InputPortCommunicationTypes;
}

const TArray<EDMXCommunicationType> FDMXProtocolSACN::GetOutputPortCommunicationTypes() const
{
	return OutputPortCommunicationTypes;
}

int32 FDMXProtocolSACN::GetMinUniverseID() const
{
	return ACN_MIN_UNIVERSE;
}

int32 FDMXProtocolSACN::GetMaxUniverseID() const
{
	return ACN_MAX_UNIVERSE;
}

bool FDMXProtocolSACN::IsValidUniverseID(int32 UniverseID) const
{
	return
		UniverseID >= GetMinUniverseID() &&
		UniverseID <= GetMaxUniverseID();
}

int32 FDMXProtocolSACN::MakeValidUniverseID(int32 DesiredUniverseID) const
{
	return FMath::Clamp(DesiredUniverseID, static_cast<int32>(ACN_MIN_UNIVERSE), static_cast<int32>(ACN_MAX_UNIVERSE));
}

bool FDMXProtocolSACN::SupportsPrioritySettings() const
{
	return true;
}

bool FDMXProtocolSACN::RegisterInputPort(const TSharedRef<FDMXInputPort, ESPMode::ThreadSafe>& InputPort)
{
	check(!InputPort->IsRegistered());
	check(!CachedInputPorts.Contains(InputPort));

	const FString& NetworkInterfaceAddress = InputPort->GetDeviceAddress();
	const EDMXCommunicationType CommunicationType = InputPort->GetCommunicationType();

	// Try to use an existing receiver or create a new one
	TSharedPtr<FDMXProtocolSACNReceiver> Receiver = FindExistingReceiver(NetworkInterfaceAddress, CommunicationType);
	if (!Receiver.IsValid())
	{
		Receiver = FDMXProtocolSACNReceiver::TryCreate(SharedThis(this), NetworkInterfaceAddress);
	}

	if (!Receiver.IsValid())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Could not create sACN receiver for input port %ls", *InputPort->GetPortName());

		return false;
	}

	Receivers.Add(Receiver);

	Receiver->AssignInputPort(InputPort);
	CachedInputPorts.Add(InputPort);

	return true;
}

void FDMXProtocolSACN::UnregisterInputPort(const TSharedRef<FDMXInputPort, ESPMode::ThreadSafe>& InputPort)
{
	check(CachedInputPorts.Contains(InputPort));
	CachedInputPorts.Remove(InputPort);

	for (const TSharedPtr<FDMXProtocolSACNReceiver>& Receiver : TSet<TSharedPtr<FDMXProtocolSACNReceiver>>(Receivers))
	{
		if (Receiver->ContainsInputPort(InputPort))
		{
			Receiver->UnassignInputPort(InputPort);

			if (Receiver->GetNumAssignedInputPorts() == 0)
			{
				Receivers.Remove(Receiver);
			}
		}
	}
}

TArray<TSharedPtr<IDMXSender>> FDMXProtocolSACN::RegisterOutputPort(const TSharedRef<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort)
{
	check(!OutputPort->IsRegistered());
	check(!CachedOutputPorts.Contains(OutputPort));

	const FString& NetworkInterfaceAddress = OutputPort->GetDeviceAddress();
	const EDMXCommunicationType CommunicationType = OutputPort->GetCommunicationType();
	const uint8 TTL = OutputPort->GetTimeToLive();

	// Try to use an existing receiver or create a new one
	TArray<TSharedPtr<IDMXSender>> NewSenders;
	if (CommunicationType == EDMXCommunicationType::Multicast)
	{
		TSharedPtr<FDMXProtocolSACNSender> NewSender = FindExistingMulticastSender(NetworkInterfaceAddress);
		if (!NewSender.IsValid())
		{
			NewSender = FDMXProtocolSACNSender::TryCreateMulticastSender(SharedThis(this), NetworkInterfaceAddress, TTL);
		}

		if (NewSender.IsValid())
		{
			Senders.Add(NewSender);
			NewSender->AssignOutputPort(OutputPort);
			NewSenders.Add(NewSender);
		}
	}
	else if (CommunicationType == EDMXCommunicationType::Unicast)
	{
		if (OutputPort->GetDestinationAddresses().Num() == 0)
		{
			UE_LOGF(LogDMXProtocol, Warning, "Cannot create DMX Protocol sACN Sender for Output Port '%ls'. Port is set to Unicast, but does not specify any Destination Addresses.", *OutputPort->GetPortName());
		}

		for (const FString& UnicastAddress : OutputPort->GetDestinationAddresses())
		{
			TSharedPtr<FDMXProtocolSACNSender> NewSender = FindExistingUnicastSender(NetworkInterfaceAddress, UnicastAddress);
			if (!NewSender.IsValid())
			{
				NewSender = FDMXProtocolSACNSender::TryCreateUnicastSender(SharedThis(this), NetworkInterfaceAddress, UnicastAddress);
			}

			if (NewSender.IsValid())
			{
				Senders.Add(NewSender);
				NewSender->AssignOutputPort(OutputPort);
				NewSenders.Add(NewSender);
			}
		}
	}
	else
	{
		// Invalid Communication Type
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create DMX Protocol sACN Sender. The communication type specified is not supported.");
	}

	if (NewSenders.Num() > 0)
	{
		CachedOutputPorts.Add(OutputPort);
	}
	else
	{
		UE_LOGF(LogDMXProtocol, Warning, "Could not create sACN sender for Output Port '%ls'", *OutputPort->GetPortName());
	}

	return NewSenders;
}

void FDMXProtocolSACN::UnregisterOutputPort(const TSharedRef<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort)
{
	check(CachedOutputPorts.Contains(OutputPort));
	CachedOutputPorts.Remove(OutputPort);

	for (const TSharedPtr<FDMXProtocolSACNSender>& Sender : TSet<TSharedPtr<FDMXProtocolSACNSender>>(Senders))
	{
		if (Sender->ContainsOutputPort(OutputPort))
		{
			Sender->UnassignOutputPort(OutputPort);

			if (Sender->GetNumAssignedOutputPorts() == 0)
			{
				Senders.Remove(Sender);
			}
		}
	}
}

bool FDMXProtocolSACN::IsCausingLoopback(EDMXCommunicationType InCommunicationType)
{
	return InCommunicationType == EDMXCommunicationType::Multicast;
}

TSharedPtr<FDMXProtocolSACNSender> FDMXProtocolSACN::FindExistingMulticastSender(const FString& NetworkInterfaceAddress) const
{
	for (const TSharedPtr<FDMXProtocolSACNSender>& Sender : Senders)
	{
		if (Sender->EqualsEndpoint(NetworkInterfaceAddress, NetworkInterfaceAddress))
		{
			return Sender;
		}
	}

	return nullptr;
}

TSharedPtr<FDMXProtocolSACNSender> FDMXProtocolSACN::FindExistingUnicastSender(const FString& NetworkInterfaceAddress, const FString& DestinationAddress) const
{
	for (const TSharedPtr<FDMXProtocolSACNSender>& Sender : Senders)
	{
		if (Sender->EqualsEndpoint(NetworkInterfaceAddress, DestinationAddress))
		{
			return Sender;
		}
	}

	return nullptr;
}

TSharedPtr<FDMXProtocolSACNReceiver> FDMXProtocolSACN::FindExistingReceiver(const FString& IPAddress, EDMXCommunicationType CommunicationType) const
{
	for (const TSharedPtr<FDMXProtocolSACNReceiver>& Receiver : Receivers)
	{
		if (Receiver->EqualsEndpoint(IPAddress))
		{
			return Receiver;
		}
	}

	return nullptr;
}
