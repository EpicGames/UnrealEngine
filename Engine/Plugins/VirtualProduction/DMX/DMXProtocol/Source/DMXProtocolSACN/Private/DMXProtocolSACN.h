// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IDMXProtocol.h"

#include "CoreMinimal.h"

class FDMXInputPort;
class FDMXOutputPort;
class FDMXProtocolSACNReceiver;
class FDMXProtocolSACNSender;


class DMXPROTOCOLSACN_API FDMXProtocolSACN
	: public IDMXProtocol
{
public:
	explicit FDMXProtocolSACN(const FName& InProtocolName);

public:
	//~ Begin IDMXProtocolBase implementation
	virtual bool Init() override;
	virtual bool Shutdown() override;
	virtual bool Tick(float DeltaTime) override;
	//~ End IDMXProtocolBase implementation

public:
	// ~Begin IDMXProtocol implementation
	virtual bool IsEnabled() const override;
	virtual const FName& GetProtocolName() const override;
	virtual const TArray<EDMXCommunicationType> GetInputPortCommunicationTypes() const override;
	virtual const TArray<EDMXCommunicationType> GetOutputPortCommunicationTypes() const override;
	virtual int32 GetMinUniverseID() const override;
	virtual int32 GetMaxUniverseID() const override;
	virtual bool IsValidUniverseID(int32 UniverseID) const override;
	virtual int32 MakeValidUniverseID(int32 DesiredUniverseID) const override;
	virtual bool SupportsPrioritySettings() const override;
	virtual bool RegisterInputPort(const TSharedRef<FDMXInputPort, ESPMode::ThreadSafe>& InputPort) override;
	virtual void UnregisterInputPort(const TSharedRef<FDMXInputPort, ESPMode::ThreadSafe>& InputPort) override;
	virtual TArray<TSharedPtr<IDMXSender>>  RegisterOutputPort(const TSharedRef<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort) override;
	virtual bool IsCausingLoopback(EDMXCommunicationType InCommunicationType) override;
	virtual void UnregisterOutputPort(const TSharedRef<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort) override;
	// ~End IDMXProtocol implementation

	/** The supported input port communication types */
	static const TArray<EDMXCommunicationType> InputPortCommunicationTypes;

	/** The supported output port communication types */
	static const TArray<EDMXCommunicationType> OutputPortCommunicationTypes;

private:
	/** Returns the sender for specified endpoint or nullptr if none exists */
	TSharedPtr<FDMXProtocolSACNSender> FindExistingMulticastSender(const FString& NetworkInterfaceAddress) const;

	/** Returns the sender for specified endpoint or nullptr if none exists */
	TSharedPtr<FDMXProtocolSACNSender> FindExistingUnicastSender(const FString& NetworkInterfaceAddress, const FString& DestinationAddress) const;

	/** Returns the receiver for specified endpoint or nullptr if none exists */
	TSharedPtr<FDMXProtocolSACNReceiver> FindExistingReceiver(const FString& IPAddress, EDMXCommunicationType CommunicationType) const;

	/** Set of all DMX input ports currently in use */
	TSet<FDMXInputPortSharedRef> CachedInputPorts;

	/** Set of all DMX input ports currently in use */
	TSet<FDMXOutputPortSharedRef> CachedOutputPorts;

	/** Set of receivers currently in use */
	TSet<TSharedPtr<FDMXProtocolSACNReceiver>> Receivers;

	/** Set of receivers currently in use */
	TSet<TSharedPtr<FDMXProtocolSACNSender>> Senders;

private:
	FName ProtocolName;
};
