// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DMXProtocolTypes.h"

#define DMX_PROTOCOLNAME_SACN "sACN"

enum 
{
	VECTOR_ROOT_E131_DATA = 0x00000004,
	VECTOR_ROOT_E131_EXTENDED = 0x00000008,
	VECTOR_E131_EXTENDED_SYNCHRONIZATION = 0x00000001,
	VECTOR_E131_EXTENDED_DISCOVERY = 0x00000002,
	VECTOR_E131_DATA_PACKET = 0x00000002,
	VECTOR_DMP_SET_PROPERTY = 0x02
};

enum
{
	ACN_PORT = 5568,
	ACN_UNIVERSE_IP_0 = 239,
	ACN_UNIVERSE_IP_1 = 255
};

enum
{
	ACN_MIN_UNIVERSE = 1,
	ACN_MAX_UNIVERSE = 63999
};

enum
{
	ACN_HEADER_SIZE = 126,
	ACN_DMX_SIZE = 512,
	ACN_IDENTIFIER_SIZE = 12,
	ACN_RLP_PREAMBLE_SIZE = 16,
	ACN_CIDBYTES = 16,
	ACN_SOURCE_NAME_SIZE = 64,
	ACN_ADDRESS_INC = 1,
	ACN_ADDRESS_ROOT_VECTOR = 18,
};

enum
{
	ACN_DMX_PACKAGE_SIZE = 638,
	ACN_DMX_ROOT_PACKAGE_SIZE = 38,
	ACN_DMX_PDU_FRAMING_PACKAGE_SIZE = 77,
	ACN_DMX_PDU_DMP_PACKAGE_SIZE = 523,
	ACN_DMX_UDP_FRAMING_PACKAGE_SIZE = 74,
	ACN_DMX_UDP_DISCOVERY_PACKAGE_SIZE = 1032,
	ACN_DMX_MIN_PACKAGE_SIZE = ACN_DMX_ROOT_PACKAGE_SIZE + ACN_DMX_PDU_FRAMING_PACKAGE_SIZE + (ACN_DMX_PDU_DMP_PACKAGE_SIZE - ACN_DMX_SIZE),
	ACN_DMX_ROOT_FLAGS_AND_LENGTH_SIZE = 622,
	ACN_DMX_FRAMING_FLAGS_AND_LENGTH_SIZE = 600,
	ACN_DMX_DMP_FLAGS_AND_LENGTH_SIZE = 523
};

enum
{
	VHD_L_FLAG = 0x80,
	VHD_V_FLAG = 0x40,
	VHD_H_FLAG = 0x20,
	VHD_D_FLAG = 0x10
};

enum
{
	ACN_DMX_START_CODE_PER_ADDRESS_PRIORITY = 0xDD
};
