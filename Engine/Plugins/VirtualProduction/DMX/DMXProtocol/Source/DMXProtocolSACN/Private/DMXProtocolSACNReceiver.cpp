// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolSACNReceiver.h"

#include "DMXProtocolLog.h"
#include "DMXProtocolSACN.h"
#include "DMXProtocolSACNConstants.h"
#include "DMXProtocolSACNUtils.h"
#include "DMXProtocolUtils.h"
#include "DMXStats.h"
#include "IO/DMXInputPort.h"
#include "Packets/DMXProtocolE131PDUPacket.h"

#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Common/UdpSocketBuilder.h"
#include "HAL/RunnableThread.h"
#include "Serialization/ArrayReader.h"

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("sACN Packages Received Total"), STAT_SACNPackagesReceived, STATGROUP_DMX);


FDMXProtocolSACNReceiver::FDMXProtocolSACNReceiver(const TSharedPtr<FDMXProtocolSACN, ESPMode::ThreadSafe>& InSACNProtocol, FSocket& InSocket, TSharedRef<FInternetAddr> InEndpointInternetAddr)
	: Protocol(InSACNProtocol)
	, Socket(&InSocket)
	, EndpointInternetAddr(InEndpointInternetAddr)
	, bStopping(false)
	, Thread(nullptr)
{
	check(Socket->GetSocketType() == SOCKTYPE_Datagram);

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	ReceivedSenderInternetAddr = SocketSubsystem->CreateInternetAddr();
	ReceivedDestinationInternetAddr = SocketSubsystem->CreateInternetAddr();

	FString ReceiverThreadName = FString(TEXT("SACNReceiver_")) + InEndpointInternetAddr->ToString(true);
	Thread = FRunnableThread::Create(this, *ReceiverThreadName, 0, TPri_TimeCritical, FPlatformAffinity::GetPoolThreadMask());

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Created sACN Receiver at %ls", *EndpointInternetAddr->ToString(false));
}

FDMXProtocolSACNReceiver::~FDMXProtocolSACNReceiver()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}

	if (Socket)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		SocketSubsystem->DestroySocket(Socket);
	}

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Destroyed sACN Receiver at %ls", *EndpointInternetAddr->ToString(false));
}

TSharedPtr<FDMXProtocolSACNReceiver> FDMXProtocolSACNReceiver::TryCreate(const TSharedPtr<FDMXProtocolSACN, ESPMode::ThreadSafe>& SACNProtocol, const FString& IPAddress)
{
	const TSharedPtr<FInternetAddr> EndpointInternetAddr = FDMXProtocolUtils::CreateInternetAddr(IPAddress, ACN_PORT);
	if (!EndpointInternetAddr.IsValid())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create sACN receiver: Invalid IP address: %ls", *IPAddress);
		return nullptr;
	}
	const FIPv4Endpoint Endpoint(EndpointInternetAddr);

	FSocket* NewListeningSocket = FUdpSocketBuilder(TEXT("UDPSACNListeningSocket"))
		.AsBlocking()
		.AsReusable()
#if PLATFORM_WINDOWS
		.BoundToAddress(Endpoint.Address)
#endif
		.BoundToPort(ACN_PORT)
#if PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP
		.WithMulticastLoopback()
		.WithMulticastTtl(1)
		.WithMulticastInterface(Endpoint.Address);
#endif
	;

	if (!NewListeningSocket)
	{
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create sACN receiver: Error create ListeningSocket for: %ls", *IPAddress);
		return nullptr;
	}

	if (!NewListeningSocket->SetIpPktInfo(true))
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		SocketSubsystem->DestroySocket(NewListeningSocket);

		UE_LOGF(LogDMXProtocol, Warning, "Cannot create sACN receiver: Platform does not support IP_PKTINFO required for sACN.");
		return nullptr;
	}

	TSharedPtr<FDMXProtocolSACNReceiver> NewReceiver = MakeShareable(new FDMXProtocolSACNReceiver(SACNProtocol, *NewListeningSocket, EndpointInternetAddr.ToSharedRef()));

	return NewReceiver;
}

bool FDMXProtocolSACNReceiver::EqualsEndpoint(const FString& IPAddress) const
{
	TSharedPtr<FInternetAddr> OtherEndpointInternetAddr = FDMXProtocolUtils::CreateInternetAddr(IPAddress, ACN_PORT);
	if (OtherEndpointInternetAddr.IsValid() && OtherEndpointInternetAddr->CompareEndpoints(*EndpointInternetAddr))
	{
		return true;
	}

	return false;
}

void FDMXProtocolSACNReceiver::AssignInputPort(const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort)
{
	check(!AssignedInputPorts.Contains(InputPort));

	const FScopeLock LockInputPorts(&AccessInputPortsMutex);
	AssignedInputPorts.Add(InputPort);

	// Join the multicast groups for the ports where needed
	uint16 UniverseIDStart = InputPort->GetExternUniverseStart();
	uint16 UniverseIDEnd = InputPort->GetExternUniverseEnd();

	TArray<uint16> ExistingMulticastGroupUniverseIDs;
	MulticastGroupAddrToUniverseIDMap.GenerateValueArray(ExistingMulticastGroupUniverseIDs);


	for (int32 UniverseID = UniverseIDStart; UniverseID <= UniverseIDEnd; UniverseID++)
	{
		if (!ExistingMulticastGroupUniverseIDs.Contains(UniverseID))
		{
			ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
			TSharedRef<FInternetAddr> NewMulticastGroupAddr = SocketSubsystem->CreateInternetAddr();

			uint32 MulticastIp = FDMXProtocolSACNUtils::GetIpForUniverseID(UniverseID);
			NewMulticastGroupAddr->SetIp(MulticastIp);
			NewMulticastGroupAddr->SetPort(ACN_PORT);

			Socket->JoinMulticastGroup(*NewMulticastGroupAddr);

			MulticastGroupAddrToUniverseIDMap.Add(MulticastIp, UniverseID);
		}
	}
}

void FDMXProtocolSACNReceiver::UnassignInputPort(const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort)
{
	check(AssignedInputPorts.Contains(InputPort));

	const FScopeLock LockInputPorts(&AccessInputPortsMutex);
	AssignedInputPorts.Remove(InputPort);

	// Leave multicast groups outside of remaining port's universe ranges
	for (const TTuple<uint32, uint16>& MulticastGroupAddrToUniverseIDKvp : TMap<uint32, uint16>(MulticastGroupAddrToUniverseIDMap))
	{
		const uint32 MulticastGroupAddr = MulticastGroupAddrToUniverseIDKvp.Key;
		const uint16 UniverseID = MulticastGroupAddrToUniverseIDKvp.Value;
		const bool bUniverseStillInUse = AssignedInputPorts.Array().ContainsByPredicate([UniverseID](const FDMXInputPortSharedPtr& OtherInputPort)
			{
				return OtherInputPort->IsExternUniverseInPortRange(UniverseID);
			});

		if (!bUniverseStillInUse)
		{
			ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
			TSharedRef<FInternetAddr> MulticastGroupAddrToLeave = SocketSubsystem->CreateInternetAddr();
			MulticastGroupAddrToLeave->SetIp(MulticastGroupAddr);
			MulticastGroupAddrToLeave->SetPort(ACN_PORT);

			Socket->LeaveMulticastGroup(*MulticastGroupAddrToLeave);

			MulticastGroupAddrToUniverseIDMap.Remove(MulticastGroupAddr);
		}
	}
}

bool FDMXProtocolSACNReceiver::Init()
{
	return true;
}

uint32 FDMXProtocolSACNReceiver::Run()
{
	// No receive refresh rate, it would deter the timestamp

	while (!bStopping)
	{
		Update(FTimespan::FromMilliseconds(1000.f));
	}

	return 0;
}

void FDMXProtocolSACNReceiver::Stop()
{
	bStopping = true;
}

void FDMXProtocolSACNReceiver::Exit()
{
}

void FDMXProtocolSACNReceiver::Tick()
{
	Update(FTimespan::Zero());
}

FSingleThreadRunnable* FDMXProtocolSACNReceiver::GetSingleThreadInterface()
{
	return this;
}

void FDMXProtocolSACNReceiver::Update(const FTimespan& SocketWaitTime)
{
	if (!Socket->Wait(ESocketWaitConditions::WaitForRead, SocketWaitTime))
	{
		return;
	}

	uint32 Size = 0;
	int32 NumBytesRead = 0;
	while (Socket->HasPendingData(Size))
	{
		TSharedRef<FArrayReader> Reader = MakeShared<FArrayReader>(true);

		// Use an aligned size instead of ACN_DMX_PACKAGE_SIZE
		constexpr uint32 SACNMaxReaderSize = 1024u;
		Reader->SetNumUninitialized(FMath::Min(Size, SACNMaxReaderSize));

		if (Socket->RecvFromWithPktInfo(Reader->GetData(), Reader->Num(), NumBytesRead, *ReceivedSenderInternetAddr, *ReceivedDestinationInternetAddr))
		{
			Reader->RemoveAt(NumBytesRead, Reader->Num() - NumBytesRead, EAllowShrinking::No);
			uint32 MulticastIp = 0;
			ReceivedDestinationInternetAddr->GetIp(MulticastIp);

			if (const uint16* UniverseIDPtr = MulticastGroupAddrToUniverseIDMap.Find(MulticastIp))
			{
				const uint16 UniverseID = *UniverseIDPtr;
				DistributeReceivedData(UniverseID, Reader);
			}
			// fallback to unicast (but only if the address is not a multicast one!)
			else if ((MulticastIp & 0xF0000000) != 0xE0000000)
			{
				// safely (and quickly) retrieve the Universe
				if (Reader->Num() >= ACN_DMX_ROOT_PACKAGE_SIZE + ACN_DMX_PDU_FRAMING_PACKAGE_SIZE)
				{
					// the Universe is in the last two bytes
					constexpr int32 UniverseOffset = -2;
					Reader->Seek(ACN_DMX_ROOT_PACKAGE_SIZE + ACN_DMX_PDU_FRAMING_PACKAGE_SIZE + UniverseOffset);
					Reader->SetByteSwapping(true);
					uint16 UniverseID = 0;
					*Reader << UniverseID;
					Reader->SetByteSwapping(false);
					Reader->Seek(0);

					// validate Universe
					const uint32 FakeMulticastIp = FDMXProtocolSACNUtils::GetIpForUniverseID(UniverseID);
					if (MulticastGroupAddrToUniverseIDMap.Contains(FakeMulticastIp))
					{
						DistributeReceivedData(UniverseID, Reader);
					}
				}
			}
		}
	}
}

void FDMXProtocolSACNReceiver::DistributeReceivedData(uint16 UniverseID, const TSharedRef<FArrayReader>& PacketReader)
{
#if UE_BUILD_DEBUG
	check(Protocol.IsValid());
#endif

	switch (GetRootPacketType(PacketReader))
	{
	case VECTOR_ROOT_E131_DATA:
		HandleDataPacket(UniverseID, PacketReader);
		return;
	default:
		return;
	}
}

void FDMXProtocolSACNReceiver::HandleDataPacket(uint16 UniverseID, const TSharedRef<FArrayReader>& PacketReader)
{
	check(Protocol.IsValid());

	FDMXProtocolE131RootLayerPacket IncomingDMXRootLayer;
	FDMXProtocolE131FramingLayerPacket IncomingDMXFramingLayer;
	FDMXProtocolE131DMPLayerPacket IncomingDMXDMPLayer;

	uint16 RootPayloadLength = 0;
	uint16 FramingPayloadLength = 0;
	uint16 DMPPayloadLength = 0;

	IncomingDMXRootLayer.Serialize(*PacketReader, RootPayloadLength);

	// check if the advertised length is valid
	if (RootPayloadLength + ACN_RLP_PREAMBLE_SIZE > PacketReader->Num())
	{
		return;
	}

	IncomingDMXFramingLayer.Serialize(*PacketReader, FramingPayloadLength);
	// check if the advertised length is still valid
	if (FramingPayloadLength + ACN_DMX_ROOT_PACKAGE_SIZE > PacketReader->Num())
	{
		return;
	}

	IncomingDMXDMPLayer.Serialize(*PacketReader, DMPPayloadLength);
	// check if the advertised length is still valid
	if ((DMPPayloadLength + ACN_DMX_ROOT_PACKAGE_SIZE + ACN_DMX_PDU_FRAMING_PACKAGE_SIZE) > PacketReader->Num())
	{
		return;
	}

	// discard per-address-priority StartCode (even if in the future we may want to support it)
	if (IncomingDMXDMPLayer.STARTCode == ACN_DMX_START_CODE_PER_ADDRESS_PRIORITY)
	{
		static bool PerAddressPriorityUnsupportedLogged = false;
		if (!PerAddressPriorityUnsupportedLogged)
		{
			UE_LOGF(LogDMXProtocol, Warning, "Received DMX sACN packet(s) that specify a per address priority. This is not supported in the current Engine version.");
			PerAddressPriorityUnsupportedLogged = true;
		}
		return;
	}

	// check if the number of properties is consistent
	if ((IncomingDMXDMPLayer.FirstPropertyAddress + IncomingDMXDMPLayer.PropertyValueCount) > (ACN_DMX_SIZE + 1))
	{
		return;
	}

	// E1.31 specs forces address increment to be 1
	if (IncomingDMXDMPLayer.AddressIncrement != ACN_ADDRESS_INC)
	{
		return;
	}

	// Eventually build a history of values for each universe,
	// this will allow to receive fewer values in e1.31 packets
	TArray<uint8>* UniverseData = PropertiesCacheValues.Find(UniverseID);
	if (!UniverseData)
	{
		TArray<uint8> DefaultProperties;
		DefaultProperties.AddZeroed(ACN_DMX_SIZE);
		UniverseData = &PropertiesCacheValues.Add(UniverseID, DefaultProperties);
	}

	// Copy relevant data
	FMemory::Memcpy(UniverseData->GetData() + IncomingDMXDMPLayer.FirstPropertyAddress, IncomingDMXDMPLayer.DMX.GetData(), IncomingDMXDMPLayer.PropertyValueCount - 1);
	
	FDMXSignalSharedRef DMXSignal = MakeShared<FDMXSignal, ESPMode::ThreadSafe>(FPlatformTime::Seconds(), UniverseID, IncomingDMXFramingLayer.Priority, PropertiesCacheValues[UniverseID]);
	
	AccessInputPortsMutex.Lock();
	for (const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort : AssignedInputPorts)
	{
		// Check packet priority
		if (!InputPort->CheckPriority(IncomingDMXFramingLayer.Priority))
		{
			continue;
		}
		InputPort->InputDMXSignal(DMXSignal);
	}
	AccessInputPortsMutex.Unlock();

	INC_DWORD_STAT(STAT_SACNPackagesReceived);
}

uint32 FDMXProtocolSACNReceiver::GetRootPacketType(const TSharedPtr<FArrayReader>& Buffer)
{
	uint32 Vector = 0x00000000;
	/* check for the minimal valid packet size (assuming at least one property) */
	if (Buffer->Num() > ACN_DMX_MIN_PACKAGE_SIZE)
	{
		// Get OpCode
		Buffer->Seek(ACN_ADDRESS_ROOT_VECTOR);
		*Buffer << Vector;
		Buffer->ByteSwap(&Vector, 4);

		// Reset Position
		Buffer->Seek(0);
	}

	return Vector;
}

uint32 FDMXProtocolSACNReceiver::GetIpForUniverseID(uint16 InUniverseID)
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	TSharedPtr<FInternetAddr> InternetAddr = SocketSubsystem->CreateInternetAddr();
	uint32 ReturnAddress = 0;
	FBufferArchive IP;
	uint8 IP_0 = ACN_UNIVERSE_IP_0;
	uint8 IP_1 = ACN_UNIVERSE_IP_1;
	IP << IP_0; // [x.?.?.?]
	IP << IP_1; // [x.x.?.?]
	IP.ByteSwap(&InUniverseID, sizeof(uint16));
	IP << InUniverseID;	// [x.x.x.x]

	InternetAddr->SetRawIp(IP);
	InternetAddr->GetIp(ReturnAddress);
	return ReturnAddress;
}
