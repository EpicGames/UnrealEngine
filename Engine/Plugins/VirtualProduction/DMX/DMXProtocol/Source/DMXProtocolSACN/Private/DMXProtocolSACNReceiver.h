// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolTypes.h"

#include "CoreMinimal.h"
#include "HAL/Runnable.h"
#include "Misc/ScopeLock.h"
#include "Misc/SingleThreadRunnable.h"
#include "Templates/Atomic.h"

class FDMXInputPort;
class FDMXProtocolSACN;

class FArrayReader;
class FSocket;
class FRunnableThread;
class ISocketSubsystem;
class FInternetAddr;


class FDMXProtocolSACNReceiver
	: public FRunnable
	, public FSingleThreadRunnable
{
protected:
	/** Constructor. Hidden on purpose, use Create instead. */
	FDMXProtocolSACNReceiver(const TSharedPtr<FDMXProtocolSACN, ESPMode::ThreadSafe>& InSACNProtocol, FSocket& InSocket, TSharedRef<FInternetAddr> InEndpointInternetAddr);

public:
	/** Destructor */
	virtual ~FDMXProtocolSACNReceiver();

	/** 
	 * Creates a new receiver for the specified IP address. returns null if no receiver can be created.
	 * Note: Doesn't test if another receiver on same IP already exists. Use HandlesInputPort to test other instances.
	 * If another receiver exists that handles IPAddress, reuse that instead.
	 */
	static TSharedPtr<FDMXProtocolSACNReceiver> TryCreate(const TSharedPtr<FDMXProtocolSACN, ESPMode::ThreadSafe>& SACNProtocol, const FString& IPAddress);

public:
	/** Returns true if the receiver uses specified IP address */
	bool EqualsEndpoint(const FString& IPAddress) const;

	/** Assigns an input port to be handled by this receiver */
	void AssignInputPort(const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort);

	/** Unassigns an input port from this receiver */
	void UnassignInputPort(const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort);

	/** Returns true if the input port is currently assigned to this receiver */
	bool ContainsInputPort(const TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>& InputPort) const { return AssignedInputPorts.Contains(InputPort); }

	/** Gets the num input ports currently assigned to this receiver */
	int32 GetNumAssignedInputPorts() const { return AssignedInputPorts.Num(); }

	/** Returns the input ports assigned to the receiver */
	const TSet<TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>>& GetAssignedInputPorts() const { return AssignedInputPorts; }

	/** Returns the IP address of a universe */
	UE_DEPRECATED(5.0, "Deprecated in favor of the more generic FDMXProtocolSACNUtils::GetIPForUniverseID")
	static uint32 GetIpForUniverseID(uint16 InUniverseID);

private:
	/** The input ports the receiver uses */
	TSet<TSharedPtr<FDMXInputPort, ESPMode::ThreadSafe>> AssignedInputPorts;

	/** Mutex access to the input ports array */
	FCriticalSection AccessInputPortsMutex;

protected:
	//~ Begin FRunnable implementation
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;
	//~ End FRunnable implementation

	//~ Begin FSingleThreadRunnable implementation
	virtual void Tick() override;
	virtual class FSingleThreadRunnable* GetSingleThreadInterface() override;
	//~ End FSingleThreadRunnable implementation

private:
	/** 
	 * Updates the thread with data from the socket.  
	 * The socket will wait for specified time unless data is received.
	 */
	void Update(const FTimespan& SocketWaitTime);

	/** Distributes received data to the protocol to be processed further */
	void DistributeReceivedData(uint16 UniverseID, const TSharedRef<FArrayReader>& PacketReader);

	/** Handles a received sACN data packet */
	void HandleDataPacket(uint16 UniverseID, const TSharedRef<FArrayReader>& PacketReader);

	/** Returns the type of the root packet */
	uint32 GetRootPacketType(const TSharedPtr<FArrayReader>& Buffer);

	/** Map of universes with their sender IP */
	TMap<uint32 /** Multicast Group Addr */, uint16 /** UniverseID */> MulticastGroupAddrToUniverseIDMap;

	/** The owning sACN protocol */
	TSharedPtr<FDMXProtocolSACN, ESPMode::ThreadSafe> Protocol;

	/** The network socket. */
	FSocket* Socket = nullptr;

	/** The endpoint internet addr (usually the network interface card IP Address) */
	TSharedPtr<FInternetAddr> EndpointInternetAddr;

	/** The sender when packets are received */
	TSharedPtr<FInternetAddr> ReceivedSenderInternetAddr;

	/** The destination when packets are received */
	TSharedPtr<FInternetAddr> ReceivedDestinationInternetAddr;

	/** Flag indicating that the thread is stopping. */
	TAtomic<bool> bStopping;

	/** The thread object. */
	FRunnableThread* Thread = nullptr;

	/** The receiver thread's name. */
	FString ThreadName;

	/** Cache of Universe properties */
	TMap<uint16, TArray<uint8>> PropertiesCacheValues;
};
