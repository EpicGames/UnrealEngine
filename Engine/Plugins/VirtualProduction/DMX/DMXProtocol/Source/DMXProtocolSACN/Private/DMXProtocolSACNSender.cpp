// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolSACNSender.h"

#include "DMXProtocolConstants.h"
#include "DMXProtocolLog.h"
#include "DMXProtocolSACN.h"
#include "DMXProtocolSACNReceiver.h"
#include "DMXProtocolSACNUtils.h"
#include "DMXProtocolSettings.h"
#include "DMXProtocolUtils.h"
#include "DMXStats.h"
#include "Packets/DMXProtocolE131PDUPacket.h"

#include "IMessageAttachment.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Common/UdpSocketBuilder.h"
#include "Common/UdpSocketSender.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "UObject/Class.h"

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("sACN Packages Sent Total"), STAT_SACNPackagesSent, STATGROUP_DMX);


FDMXProtocolSACNSender::FDMXProtocolSACNSender(const TSharedPtr<FDMXProtocolSACN, ESPMode::ThreadSafe>& InSACNProtocol, FSocket& InSocket, TSharedRef<FInternetAddr> InNetworkInterfaceInternetAddr, TSharedRef<FInternetAddr> InDestinationInternetAddr, const bool bInIsMulticast)
	: Protocol(InSACNProtocol)
	, Socket(&InSocket)
	, NetworkInterfaceInternetAddr(InNetworkInterfaceInternetAddr)
	, DestinationInternetAddr(InDestinationInternetAddr)
	, bIsMulticast(bInIsMulticast)
{
	check(DestinationInternetAddr.IsValid());

	if (bIsMulticast)
	{
		UE_LOGF(LogDMXProtocol, VeryVerbose, "Created sACN Multicast Sender at %ls", *NetworkInterfaceInternetAddr->ToString(false));
	}
	else
	{
		UE_LOGF(LogDMXProtocol, VeryVerbose, "Created sACN Sender at %ls sending to %ls", *NetworkInterfaceInternetAddr->ToString(false), *DestinationInternetAddr->ToString(false));
	}
}

FDMXProtocolSACNSender::~FDMXProtocolSACNSender()
{
	if (Socket)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		SocketSubsystem->DestroySocket(Socket);
	}

	UE_LOGF(LogDMXProtocol, VeryVerbose, "Destroyed sACN Sender at %ls sending to %ls", *NetworkInterfaceInternetAddr->ToString(false), *DestinationInternetAddr->ToString(false));
}

TSharedPtr<FDMXProtocolSACNSender> FDMXProtocolSACNSender::TryCreateUnicastSender(
	const TSharedPtr<FDMXProtocolSACN,
	ESPMode::ThreadSafe>& SACNProtocol,
	const FString& InNetworkInterfaceIP,
	const FString& InUnicastIP)
{
	// Try to create a socket
	TSharedPtr<FInternetAddr> NewNetworkInterfaceInternetAddr = FDMXProtocolUtils::CreateInternetAddr(InNetworkInterfaceIP, ACN_PORT);
	if (!NewNetworkInterfaceInternetAddr.IsValid())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create sACN sender: Invalid IP address: %ls", *InNetworkInterfaceIP);
		return nullptr;
	}

	const FIPv4Endpoint NewNetworkInterfaceEndpoint = FIPv4Endpoint(NewNetworkInterfaceInternetAddr);
	
	FSocket* NewSocket =
		FUdpSocketBuilder(TEXT("UDPSACNUnicastSocket"))
		.AsBlocking()
		.AsReusable()
		.BoundToEndpoint(NewNetworkInterfaceEndpoint);

	if(!NewSocket)
	{
		UE_LOGF(LogDMXProtocol, Warning, "Invalid Network Interface IP %ls for DMX Port. Please update your Output Port in Project Settings -> Plugins -> DMX Plugin", *InNetworkInterfaceIP);
		return nullptr;
	}

	// Try create the unicast internet addr
	TSharedPtr<FInternetAddr> NewUnicastInternetAddr = FDMXProtocolUtils::CreateInternetAddr(InUnicastIP, ACN_PORT);
	if (!NewUnicastInternetAddr.IsValid())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Invalid Unicast IP %ls for DMX Port. Please update your Output Port in Project Settings -> Plugins -> DMX Plugin", *InUnicastIP);
		return nullptr;
	}

	TSharedPtr<FDMXProtocolSACNSender> NewSender = MakeShareable(new FDMXProtocolSACNSender(SACNProtocol, *NewSocket, NewNetworkInterfaceInternetAddr.ToSharedRef(), NewUnicastInternetAddr.ToSharedRef(), false));

	return NewSender;
}

TSharedPtr<FDMXProtocolSACNSender> FDMXProtocolSACNSender::TryCreateMulticastSender(
	const TSharedPtr<FDMXProtocolSACN,
	ESPMode::ThreadSafe>& SACNProtocol,
	const FString& InNetworkInterfaceIP, 
	const uint8 TTL)
{
	// Try to create a socket
#if !PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP
	UE_LOGF(LogDMXProtocol, Warning, "Cannot create a multicast sACN sender. Platform doesn't support UDP multicast groups.");
	return nullptr;
#else 

	const TSharedPtr<FInternetAddr> NewNetworkInterfaceInternetAddr = FDMXProtocolUtils::CreateInternetAddr(InNetworkInterfaceIP, ACN_PORT);
	if (!NewNetworkInterfaceInternetAddr.IsValid())
	{
		UE_LOGF(LogDMXProtocol, Warning, "Cannot create sACN sender: Invalid IP address: %ls", *InNetworkInterfaceIP);
		return nullptr;
	}
	FIPv4Endpoint NewNetworkInterfaceEndpoint = FIPv4Endpoint(NewNetworkInterfaceInternetAddr);

	FSocket* NewSocket = 
		FUdpSocketBuilder(TEXT("UDPSACNMulticastSocket"))
        .AsBlocking()
        .AsReusable()
#if PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP
        .WithMulticastLoopback()
		.WithMulticastInterface(NewNetworkInterfaceEndpoint.Address)
        .WithMulticastTtl(TTL)
#endif
	;
	
	if(!NewSocket)
	{
		UE_LOGF(LogDMXProtocol, Warning, "Invalid Network Interface IP %ls for DMX Port. Please update your Output Ports in Project Settings -> Plugins -> DMX Plugin", *InNetworkInterfaceIP);
		return nullptr;
	}

	TSharedPtr<FDMXProtocolSACNSender> NewSender = MakeShareable(new FDMXProtocolSACNSender(SACNProtocol, *NewSocket, NewNetworkInterfaceInternetAddr.ToSharedRef(), NewNetworkInterfaceInternetAddr.ToSharedRef(), true));

	return NewSender;
#endif
}

bool FDMXProtocolSACNSender::EqualsEndpoint(const FString& NetworkInterfaceIP, const FString& DestinationIPAddress) const
{
	TSharedPtr<FInternetAddr> OtherNetworkInterfaceInternetAddr = FDMXProtocolUtils::CreateInternetAddr(NetworkInterfaceIP, ACN_PORT);
	if (OtherNetworkInterfaceInternetAddr.IsValid() && OtherNetworkInterfaceInternetAddr->CompareEndpoints(*NetworkInterfaceInternetAddr))
	{
		TSharedPtr<FInternetAddr> OtherDestinationInternetAddr = FDMXProtocolUtils::CreateInternetAddr(DestinationIPAddress, ACN_PORT);
		if (OtherDestinationInternetAddr.IsValid() && OtherDestinationInternetAddr->CompareEndpoints(*DestinationInternetAddr))
		{
			return true;
		}
	}

	return false;
}

void FDMXProtocolSACNSender::AssignOutputPort(const TSharedPtr<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort)
{
	check(!AssignedOutputPorts.Contains(OutputPort));
	AssignedOutputPorts.Add(OutputPort);
}

void FDMXProtocolSACNSender::UnassignOutputPort(const TSharedPtr<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort)
{
	check(AssignedOutputPorts.Contains(OutputPort));
	AssignedOutputPorts.Remove(OutputPort);
}

bool FDMXProtocolSACNSender::IsCausingLoopback() const
{
	return CommunicationType == EDMXCommunicationType::Broadcast;
}

void FDMXProtocolSACNSender::SendDMXSignal(const FDMXSignalSharedRef& DMXSignal)
{
	// This may be called from multiple threads, so calls need be synchronized here
	const FScopeLock SendDMXLock(&SendDMXCriticalSection);

	TArray<uint8> Packet;

	FDMXProtocolE131RootLayerPacket RootLayer;
	static FGuid Guid = FGuid::NewGuid();
	FMemory::Memcpy(RootLayer.CID.GetData(), &Guid, ACN_CIDBYTES);

	Packet.Append(*RootLayer.Pack(ACN_DMX_SIZE));

	int32 UniverseID = DMXSignal->ExternUniverseID;

	check(UniverseID > 0 && UniverseID <= TNumericLimits<uint16>::Max());
	FDMXProtocolE131FramingLayerPacket FramingLayer;
	FramingLayer.Universe = UniverseID;
	FramingLayer.SequenceNumber = UniverseIDToSequenceNumberMap.FindOrAdd(UniverseID, TNumericLimits<uint8>::Max())++; // Init to max, let it wrap over to 0 at first
	FramingLayer.Priority = DMXSignal->Priority;
	Packet.Append(*FramingLayer.Pack(ACN_DMX_SIZE));

	FDMXProtocolE131DMPLayerPacket DMPLayer;
	DMPLayer.AddressIncrement = ACN_ADDRESS_INC;
	DMPLayer.PropertyValueCount = ACN_DMX_SIZE + 1;
	FMemory::Memcpy(DMPLayer.DMX.GetData(), DMXSignal->ChannelData.GetData(), ACN_DMX_SIZE);

	Packet.Append(*DMPLayer.Pack(ACN_DMX_SIZE));

	const int32 SendDataSize = Packet.Num();

	// Try to send, log errors but avoid spaming the Log
	static bool bErrorEverLogged = false;
	TSharedPtr<FInternetAddr> CurrentDestination = DestinationInternetAddr;

	// if in multicast, compute the destination
	if (bIsMulticast)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		CurrentDestination = SocketSubsystem->CreateInternetAddr();

		uint32 IpForUniverse = FDMXProtocolSACNUtils::GetIpForUniverseID(UniverseID);
		CurrentDestination->SetIp(IpForUniverse);
		CurrentDestination->SetPort(ACN_PORT);
	}

	int32 BytesSent = -1;
	if (Socket->SendTo(Packet.GetData(), Packet.Num(), BytesSent, *CurrentDestination))
	{
		INC_DWORD_STAT(STAT_SACNPackagesSent);
	}
	else
	{
		if (!bErrorEverLogged)
		{
			ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
			TEnumAsByte<ESocketErrors> RecvFromError = SocketSubsystem->GetLastErrorCode();

			UE_LOGF(LogDMXProtocol, Error, "Socket subsystem failed sending DMX to %ls with Error Code %d", *DestinationInternetAddr->ToString(false), RecvFromError.GetValue());

			bErrorEverLogged = true;
		}
	}

	if (BytesSent != SendDataSize && !bErrorEverLogged)
	{
		UE_LOGF(LogDMXProtocol, Warning, "Incomplete DMX Packet sent to %ls", *DestinationInternetAddr->ToString(false));
		bErrorEverLogged = true;
	}
}
