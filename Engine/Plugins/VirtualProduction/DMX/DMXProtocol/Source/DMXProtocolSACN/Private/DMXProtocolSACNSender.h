// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXProtocolCommon.h"
#include "DMXProtocolTypes.h"
#include "Interfaces/IDMXSender.h"

#include "CoreMinimal.h"
#include "Misc/ScopeLock.h" 
#include "Serialization/ArrayWriter.h"

struct FDMXProtocolSACNRDM;

enum class EDMXCommunicationType : uint8;
class FDMXOutputPort;

class FDMXProtocolSACN;
class FInternetAddr;
class FSocket;
class ISocketSubsystem;


class DMXPROTOCOLSACN_API FDMXProtocolSACNSender
	: public IDMXSender
{
protected:
	FDMXProtocolSACNSender& operator=(const FDMXProtocolSACNSender&) = delete;
	FDMXProtocolSACNSender(const FDMXProtocolSACNSender&) = delete;

	/** Constructor. Hidden on purpose, use TryCreateUnicastSender and TryCreateMulticastSender instead. */
	FDMXProtocolSACNSender(const TSharedPtr<FDMXProtocolSACN, ESPMode::ThreadSafe>& InSACNProtocol, FSocket& InSocket, TSharedRef<FInternetAddr> InNetworkInternetAddr, TSharedRef<FInternetAddr> InDestinationInternetAddr, const bool bInIsMulticast);

public:
	/** Destructor */
	virtual ~FDMXProtocolSACNSender();

	/**
	 * Creates a new unicast sender for the specified IP address. Returns null if no sender can be created.
	 * Note: Doesn't test if another sender on same IP already exists. Use EqualsEndpoint method to test other instances.
	 * If another sender exists that handles IPAddress, reuse that instead.
	 *
	 * @param SACNProtocol			The protocol instance that wants to create the sender
	 * @param InNetworkInterfaceIP		The IP address of the network interface that is used to send data
	 * @param InUnicastIP				The IP address to unicast to.
	 */
	static TSharedPtr<FDMXProtocolSACNSender> TryCreateUnicastSender(const TSharedPtr<FDMXProtocolSACN, ESPMode::ThreadSafe>& SACNProtocol, const FString& InNetworkInterfaceIP, const FString& InUnicastIP);

	/**
	 * Creates a new multicast sender for the specified IP address. Returns null if no sender can be created.
	 * Note: Doesn't test if another sender on same IP already exists. Use EqualsEndpoint method to test other instances.
	 * If another sender exists that handles IPAddress, reuse that instead.
	 *
	 * @param SACNProtocol			The protocol instance that wants to create the sender
	 * @param InNetworkInterfaceIP		The IP address of the network interface that is used to send data
	 */
	static TSharedPtr<FDMXProtocolSACNSender> TryCreateMulticastSender(const TSharedPtr<FDMXProtocolSACN, ESPMode::ThreadSafe>& SACNProtocol, const FString& InNetworkInterfaceIP, const uint8 TTL);

	// ~Begin IDMXSender Interface
	virtual void SendDMXSignal(const FDMXSignalSharedRef& DMXSignal) override;
	// ~End IDMXSender Interface

public:
	/** Returns true if the sender causes loopbacks over the network */
	bool IsCausingLoopback() const;

	/** Returns true if the sender uses specified endpoint */
	bool EqualsEndpoint(const FString& NetworkInterfaceIP, const FString& DestinationIPAddress) const;

	/** Assigns an output port to be handled by this sender */
	void AssignOutputPort(const TSharedPtr<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort);

	/** Unassigns an output port from this sender */
	void UnassignOutputPort(const TSharedPtr<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort);

	/** Returns true if the output port is currently assigned to this sender */
	bool ContainsOutputPort(const TSharedPtr<FDMXOutputPort, ESPMode::ThreadSafe>& OutputPort) const { return AssignedOutputPorts.Contains(OutputPort); }

	/** Gets the num output ports currently assigned to this sender */
	int32 GetNumAssignedOutputPorts() const { return AssignedOutputPorts.Num(); }

	/** Returns the output ports assigned to the sender */
	const TSet<TSharedPtr<FDMXOutputPort, ESPMode::ThreadSafe>>& GetAssignedOutputPorts() const { return AssignedOutputPorts; }

private:
	/** The Output ports the receiver uses */
	TSet<TSharedPtr<FDMXOutputPort, ESPMode::ThreadSafe>> AssignedOutputPorts;

	/** Map of universes with their current sequence number */
	TMap<int32 /** Universe ID */, uint8 /** Sequence Number */> UniverseIDToSequenceNumberMap;

	/** The sACN protocol instance */
	TSharedPtr<FDMXProtocolSACN, ESPMode::ThreadSafe> Protocol;

	/** Holds the network socket used to sender packages. */
	FSocket* Socket = nullptr;

	/** Critical section to be used during SendDMXSignal, as it may be called concurrently from many threads */
	FCriticalSection SendDMXCriticalSection;

	/** The network interface internet addr */
	TSharedPtr<FInternetAddr> NetworkInterfaceInternetAddr;

	/** The endpoint interface internet addr */
	TSharedPtr<FInternetAddr> DestinationInternetAddr;

	/** Communication type used for the network traffic */
	EDMXCommunicationType CommunicationType;

	/** Is a Multicast sender ? */
	const bool bIsMulticast;
};
