// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"


struct FDMXProtocolSACNSequenceNumber
{
	FDMXProtocolSACNSequenceNumber()
		: Curr(0x00)
	{}

	uint8 Increment()
	{
		if (Curr == 0xFF)
		{
			Curr = 0x00;
		}

		return Curr++;
	}

private:
	uint8 Curr;
};
