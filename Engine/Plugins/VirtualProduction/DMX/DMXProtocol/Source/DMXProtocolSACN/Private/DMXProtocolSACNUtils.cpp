// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXProtocolSACNUtils.h"

#include "IPAddress.h"
#include "SocketSubsystem.h"
#include "Serialization/BufferArchive.h"


uint32 FDMXProtocolSACNUtils::GetIpForUniverseID(uint16 InUniverseID)
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	TSharedPtr<FInternetAddr> InternetAddr = SocketSubsystem->CreateInternetAddr();
	uint32 ReturnAddress = 0;
	FBufferArchive IP;
	uint8 IP_0 = ACN_UNIVERSE_IP_0;
	uint8 IP_1 = ACN_UNIVERSE_IP_1;
	IP << IP_0; // [x.?.?.?]
	IP << IP_1; // [x.x.?.?]
	IP.ByteSwap(&InUniverseID, sizeof(uint16));
	IP << InUniverseID;	// [x.x.x.x]

	InternetAddr->SetRawIp(IP);
	InternetAddr->GetIp(ReturnAddress);
	return ReturnAddress;
}
