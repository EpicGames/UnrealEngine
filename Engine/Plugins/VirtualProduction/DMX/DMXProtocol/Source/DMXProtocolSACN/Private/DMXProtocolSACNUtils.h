// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DMXProtocolTypes.h"
#include "DMXProtocolSACNConstants.h"

#include "Serialization/ArrayReader.h"


class FDMXProtocolSACNUtils
{
public:
	/** Returns the SACN specific IP for a Universe ID */
	static uint32 GetIpForUniverseID(uint16 InUniverseID);

	// Can't instantiate this class
	FDMXProtocolSACNUtils() = delete;
};
