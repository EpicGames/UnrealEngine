// Copyright Epic Games, Inc. All Rights Reserved.

#include "Packets/DMXProtocolE131PDUPacket.h"
#include "DMXProtocolTypes.h"
#include "Serialization/BufferArchive.h"

#include "DMXProtocolMacros.h"

TSharedPtr<FBufferArchive> FDMXProtocolE131RootLayerPacket::Pack(uint16 PropertiesNum)
{
	TSharedPtr<FBufferArchive> Writer = MakeShared<FBufferArchive>();
	uint16 PayloadLength = ACN_DMX_MIN_PACKAGE_SIZE - ACN_RLP_PREAMBLE_SIZE + PropertiesNum;
	Serialize(*Writer, PayloadLength);
	return Writer;
}

void FDMXProtocolE131RootLayerPacket::Serialize(FArchive& Ar, uint16& PayloadLength)
{
	Ar.SetByteSwapping(true);
	Ar << PreambleSize;
	Ar << PostambleSize;
	Ar.SetByteSwapping(false);
	Ar.Serialize(reinterpret_cast<void*>(ACNPacketIdentifier.GetData()), ACN_IDENTIFIER_SIZE);
	Ar.SetByteSwapping(true);
	if (Ar.IsLoading())
	{
		Ar << FlagsAndLength;
		PayloadLength = FlagsAndLength & 0x0fff;
	}
	else
	{
		FlagsAndLength = 0x7000 | (PayloadLength & 0x0fff);
		Ar << FlagsAndLength;
	}
	Ar << Vector;
	Ar.SetByteSwapping(false);
	Ar.Serialize(reinterpret_cast<void*>(CID.GetData()), ACN_CIDBYTES);
}


TSharedPtr<FBufferArchive> FDMXProtocolE131FramingLayerPacket::Pack(uint16 PropertiesNum)
{
	TSharedPtr<FBufferArchive> Writer = MakeShared<FBufferArchive>();
	uint16 PayloadLength = ACN_DMX_MIN_PACKAGE_SIZE - ACN_DMX_ROOT_PACKAGE_SIZE + PropertiesNum;
	Serialize(*Writer, PayloadLength);
	return Writer;
}

void FDMXProtocolE131FramingLayerPacket::Serialize(FArchive& Ar, uint16& PayloadLength)
{
	Ar.SetByteSwapping(true);
	if (Ar.IsLoading())
	{
		Ar << FlagsAndLength;
		PayloadLength = FlagsAndLength & 0x0fff;
	}
	else
	{
		FlagsAndLength = 0x7000 | (PayloadLength & 0x0fff);
		Ar << FlagsAndLength;
	}
	Ar << Vector;
	Ar.SetByteSwapping(false);
	Ar.Serialize(reinterpret_cast<void*>(SourceName.GetData()), ACN_SOURCE_NAME_SIZE);
	Ar.SetByteSwapping(true);
	Ar << Priority;
	Ar << SynchronizationAddress;
	Ar << SequenceNumber;
	Ar << Options;
	Ar << Universe;
}


TSharedPtr<FBufferArchive> FDMXProtocolE131DMPLayerPacket::Pack(uint16 PropertiesNum)
{
	TSharedPtr<FBufferArchive> Writer = MakeShared<FBufferArchive>();
	uint16 PayloadLength = ACN_DMX_MIN_PACKAGE_SIZE - ACN_DMX_ROOT_PACKAGE_SIZE - ACN_DMX_PDU_FRAMING_PACKAGE_SIZE + PropertiesNum;
	Serialize(*Writer, PayloadLength);
	return Writer;
}

void FDMXProtocolE131DMPLayerPacket::Serialize(FArchive& Ar, uint16& PayloadLength)
{
	Ar.SetByteSwapping(true);
	if (Ar.IsLoading())
	{
		Ar << FlagsAndLength;
		PayloadLength = FlagsAndLength & 0x0fff;
	}
	else
	{
		FlagsAndLength = 0x7000 | (PayloadLength & 0x0fff);
		Ar << FlagsAndLength;
	}
	Ar << Vector;
	Ar << AddressTypeAndDataType;
	Ar << FirstPropertyAddress;
	Ar << AddressIncrement;
	Ar << PropertyValueCount;
	Ar << STARTCode;
	Ar.SetByteSwapping(false);

	// we are allowed to send/receive less properties
	Ar.Serialize(reinterpret_cast<void*>(DMX.GetData()), PropertyValueCount - 1);
}
