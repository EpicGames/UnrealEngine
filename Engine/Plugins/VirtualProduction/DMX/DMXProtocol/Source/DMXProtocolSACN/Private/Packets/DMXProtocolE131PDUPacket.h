// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IDMXProtocolPacket.h"
#include "DMXProtocolSACNConstants.h"

struct FDMXProtocolE131RootLayerPacket
	: public IDMXProtocolPacket
{
public:
	//~ Begin IDMXProtocolPacket implementation
	virtual TSharedPtr<FBufferArchive> Pack(const uint16 PropertiesNum) override;
	//~ End IDMXProtocolPort implementation

	void Serialize(FArchive& Ar, uint16& PayloadLength);

public:
	uint16 PreambleSize = ACN_RLP_PREAMBLE_SIZE;
	uint16 PostambleSize = 0;
	TArray<uint8, TInlineAllocator<ACN_IDENTIFIER_SIZE>> ACNPacketIdentifier = { 'A', 'S', 'C', '-', 'E', '1', '.', '1', '7', '\0', '\0', '\0' };
	uint16 FlagsAndLength = ACN_DMX_ROOT_FLAGS_AND_LENGTH_SIZE;
	uint32 Vector = VECTOR_ROOT_E131_DATA;
	TArray<uint8, TInlineAllocator<ACN_CIDBYTES>> CID = { 0 };
};

struct FDMXProtocolE131FramingLayerPacket
	: public IDMXProtocolPacket
{
public:
	//~ Begin IDMXProtocolPacket implementation
	virtual TSharedPtr<FBufferArchive> Pack(const uint16 PropertiesNum) override;
	//~ End IDMXProtocolPort implementation

	void Serialize(FArchive& Ar, uint16& PayloadLength);

public:
	uint16 FlagsAndLength = ACN_DMX_FRAMING_FLAGS_AND_LENGTH_SIZE;
	uint32 Vector = VECTOR_E131_DATA_PACKET;
	TArray<uint8, TInlineAllocator<ACN_SOURCE_NAME_SIZE>> SourceName = { 'U', 'E', '5', '\0' };
	uint8 Priority = 1;
	uint16 SynchronizationAddress = 0;
	uint8 SequenceNumber = 0;
	uint8 Options = 0;
	uint16 Universe = 0;
};

struct FDMXProtocolE131DMPLayerPacket
	: public IDMXProtocolPacket
{
public:
	FDMXProtocolE131DMPLayerPacket()
	{
		// preinitialize the whole DMX universe with 0
		DMX.AddZeroed(ACN_DMX_SIZE);
	}

	//~ Begin IDMXProtocolPacket implementation
	virtual TSharedPtr<FBufferArchive> Pack(const uint16 PropertiesNum) override;
	//~ End IDMXProtocolPort implementation

	void Serialize(FArchive& Ar, uint16& PayloadLength);

public:
	uint16 FlagsAndLength = ACN_DMX_DMP_FLAGS_AND_LENGTH_SIZE;
	uint8 Vector = VECTOR_DMP_SET_PROPERTY;
	uint8 AddressTypeAndDataType = 0xa1; // The const from SACN protocol documentation
	uint16 FirstPropertyAddress = 0;
	uint16 AddressIncrement = 0;
	uint16 PropertyValueCount = 0;
	uint8 STARTCode = 0;
	TArray<uint8, TInlineAllocator<ACN_DMX_SIZE>> DMX;
};

