// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IDMXProtocolFactory.h"

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleInterface.h"

struct FDMXProtocolRegistrationParams;


/**
 */
class FDMXProtocolFactorySACN : public IDMXProtocolFactory
{
public:
	virtual IDMXProtocolPtr CreateProtocol(const FName& ProtocolName) override;
};

/**
 */
class DMXPROTOCOLSACN_API FDMXProtocolSACNModule : public IModuleInterface
{
private:

	/** Class responsible for creating instance(s) of the protocol */
	TUniquePtr<FDMXProtocolFactorySACN> FactorySACN;

public:
	UE_DEPRECATED(4.27, "Use DMX_PROTOCOLNAME_SACN instead, see DMXProtocolSACNConstants.h")
	static FName const NAME_SACN;

public:
	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface implementation

	FDMXProtocolSACNModule& Get();

private:
	/** Registers this sACN implementation with the Protocol Module */
	void RegisterWithProtocolModule(TArray<FDMXProtocolRegistrationParams>& InOutProtocolRegistrationParamsArray);

	/**
	 * Sending DMX through console command
	 * Command structure is DMX.SACN.SendDMX [UniverseID] Channel:Value Channel:Value Channel:Value ...
	 * Example:
	 * DMX.SACN.SendDMX 7 25:156 26:0 27:10 28:15
	 * It will send the DMX to Universe 7. It could be any value from 0 to 64214
	 * And it update the channels values for channel 25, 26, 27, 28 with values 156, 0, 10, 15
	 * Channel could be any value from 0 to 511
	 * Value could be any value from 0 to 255
	 */
	static void SendDMXCommandHandler(const TArray<FString>& Args);

	/**
	 * Reset DMX through console command
	 * Command structure is DMX.SACN.ResetDMXSend [UniverseID]
	 * Example:
	 * DMX.SACN.ResetDMXSend 7
	 */
	static void ResetDMXSendUniverseHandler(const TArray<FString>& Args);

private:
	/** Command for sending DMX through the console */
	static FAutoConsoleCommand SendDMXCommand;

	/** Command for reset DMX universe through the console */
	static FAutoConsoleCommand ResetDMXSendUniverseCommand;
};
